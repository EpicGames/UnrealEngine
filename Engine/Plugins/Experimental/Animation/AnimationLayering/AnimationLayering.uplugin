{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Animation Layering",
	"Description": "",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AnimationLayering",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AnimationLayeringUncookedOnly",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "PoseSearch",
			"Enabled": true
		},
		{
			"Name": "AnimationModifierLibrary",
			"Enabled": true
		}
	]
}