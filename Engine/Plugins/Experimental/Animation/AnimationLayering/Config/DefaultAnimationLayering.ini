[CoreRedirects]
; Moved from AnimationLayeringRuntime to AnimationLayering
+StructRedirects=(OldName="/Script/AnimationLayeringRuntime.AnimNode_BoneMask", NewName="/Script/AnimationLayering.AnimNode_BoneMask")
+StructRedirects=(OldName="/Script/AnimationLayeringRuntime.AnimNode_CopyBoneAdvanced", NewName="/Script/AnimationLayering.AnimNode_CopyBoneAdvanced")
+StructRedirects=(OldName="/Script/AnimationLayeringRuntime.BoneMask", NewName="/Script/AnimationLayering.BoneMask")
+StructRedirects=(OldName="/Script/AnimationLayeringRuntime.BoneMaskBodyPartDefinition", NewName="/Script/AnimationLayering.BoneMaskBodyPartDefinition")
+StructRedirects=(OldName="/Script/AnimationLayeringRuntime.BoneMaskBodyPartNameContainer", NewName="/Script/AnimationLayering.BoneMaskBodyPartNameContainer")
+StructRedirects=(OldName="/Script/AnimationLayeringRuntime.BoneMaskDefinition", NewName="/Script/AnimationLayering.BoneMaskDefinition")
+StructRedirects=(OldName="/Script/AnimationLayeringRuntime.BoneMaskEntry", NewName="/Script/AnimationLayering.BoneMaskEntry")
+StructRedirects=(OldName="/Script/AnimationLayeringRuntime.BoneMaskPerBoneData", NewName="/Script/AnimationLayering.BoneMaskPerBoneData")
+StructRedirects=(OldName="/Script/AnimationLayeringRuntime.BoneMaskUpdateMultiParam", NewName="/Script/AnimationLayering.BoneMaskUpdateMultiParam")
+ClassRedirects=(OldName="/Script/AnimationLayeringRuntime.BoneMaskDefinitionDataAsset", NewName="/Script/AnimationLayering.BoneMaskDefinitionDataAsset")

; Moved from AnimationLayeringEditor to AnimationLayeringUncookedOnly
+ClassRedirects=(OldName="/Script/AnimationLayeringEditor.AnimGraphNode_BoneMask", NewName="/Script/AnimationLayeringUncookedOnly.AnimGraphNode_BoneMask")
+ClassRedirects=(OldName="/Script/AnimationLayeringEditor.AnimGraphNode_CopyBoneAdvanced", NewName="/Script/AnimationLayeringUncookedOnly.AnimGraphNode_CopyBoneAdvanced")

