// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AnimationLayering : ModuleRules
{
	public AnimationLayering(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"AnimationCore",
				"AnimGraphRuntime",
				// ... add other public dependencies that you statically link with here ...
			}
			);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"PoseSearch",
				// ... add private dependencies that you statically link with here ...	
			}
			);
	}
}
