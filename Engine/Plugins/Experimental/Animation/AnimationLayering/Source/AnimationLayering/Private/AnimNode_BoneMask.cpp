// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_BoneMask.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimCurveUtils.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_BoneMask)

namespace UE::Anim::BoneMask
{
	// Lazy component-space pose container for rotations ONLY. Calculated on demand.
	template<typename InAllocatorType>
	struct FCompactPoseRotationsCS
	{
		TArray<FQuat, InAllocatorType> BoneRotationsCS;
		TBitArray<InAllocatorType> ComponentSpaceMask;

		void SetBoneRotationCS(const FCompactPoseBoneIndex& BoneIndex, const FQuat& NewRotationCS)
		{
			const int32 Idx = BoneIndex.GetInt();
			ComponentSpaceMask[Idx] = true;
			BoneRotationsCS[Idx] = NewRotationCS;
		}

		void InitPose(const FCompactPose& LocalPose, const bool bRootSpace = false)
		{
			// Reset our containers without resizing.
			BoneRotationsCS.Reset();
			ComponentSpaceMask.Reset();

			const int32 NumBones = LocalPose.GetNumBones();
			// This will only be slow on the first init.
			BoneRotationsCS.AddUninitialized(NumBones);
			ComponentSpaceMask.Reserve(NumBones);

			// If we want mesh space, copy the root bone since it's always in mesh space.
			BoneRotationsCS[0] = bRootSpace ? FQuat::Identity : LocalPose[FCompactPoseBoneIndex(0)].GetRotation();
			ComponentSpaceMask.Add(true);
			// Every other bone needs to be lazy calculated
			ComponentSpaceMask.Add(false, NumBones - 1);
		}

		const FQuat& SafeGetBoneRotationCS(const FCompactPoseBoneIndex BoneIndex, const FCompactPose& LocalPose)
		{
			const int32 Idx = BoneIndex.GetInt();

			if (ensureMsgf(ComponentSpaceMask.IsValidIndex(Idx), TEXT("BoneMask::FCompactPoseRotationsCS. SafeGetBoneRotationCS. Bone Index '%d' is not a valid index in ComponentSpaceMask with Num: %d"),
				Idx, ComponentSpaceMask.Num()))
			{
				if (ComponentSpaceMask[Idx] == false)
				{
					CalculateRotationCS(BoneIndex, LocalPose);
				}
			}

			return UnsafeGetBoneRotationCS(BoneIndex);
		}

		// Only use if the component space rotaiton is guaranteed to have been calculated already.
		FORCEINLINE const FQuat& UnsafeGetBoneRotationCS(const FCompactPoseBoneIndex BoneIndex)
		{
			if (ensureMsgf(BoneRotationsCS.IsValidIndex(BoneIndex.GetInt()), TEXT("BoneMask::FCompactPoseRotationsCS. UnsafeGetBoneRotationCS Bone Index '%d' is not a valid index BoneRotationsCS with Num: %d"),
				BoneIndex.GetInt(), BoneRotationsCS.Num()))
			{
				return BoneRotationsCS[BoneIndex.GetInt()];
			}
			else
			{
				return FQuat::Identity;
			}
		}

		void CalculateRotationCS(const FCompactPoseBoneIndex BoneIndex, const FCompactPose& LocalPose)
		{
			if (ensureMsgf(LocalPose.IsValid(), TEXT("BoneMask::FCompactPoseRotationsCS. LocalPose is not valid")))
			{
				if (ensureMsgf(LocalPose.IsValidIndex(BoneIndex), TEXT("BoneMask::FCompactPoseRotationsCS. Bone Index '%d' is not a valid index in Local Pose with Num Bones: %d"),
					BoneIndex.GetInt(), LocalPose.GetNumBones()))
				{
					// root is already calculated on init, so root should not come here therefore parent index is safe to use.
					const FCompactPoseBoneIndex ParentIndex = LocalPose.GetParentBoneIndex(BoneIndex);
					if (ensureMsgf(LocalPose.IsValidIndex(ParentIndex), TEXT("BoneMask::FCompactPoseRotationsCS. Bone Index '%d' Parent of '%d' is not a valid index in Local Pose with Num Bones: %d"),
						ParentIndex.GetInt(), BoneIndex.GetInt(), LocalPose.GetNumBones()))
					{
						const int32 ParentBoneIdx = ParentIndex.GetInt();
						if (ensureMsgf(ComponentSpaceMask.IsValidIndex(ParentBoneIdx), TEXT("BoneMask::FCompactPoseRotationsCS. Bone Index '%d' Parent of '%d' is not a valid index in ComponentSpaceMask with Num: %d"),
							ParentBoneIdx, BoneIndex.GetInt(), ComponentSpaceMask.Num()))
						{
							// if parent already has been calculated, use it
							if (ComponentSpaceMask[ParentBoneIdx] == false)
							{
								// if parent hasn't been calculated, calculate parents recursively.
								CalculateRotationCS(ParentIndex, LocalPose);
							}

							const int32 BoneIdx = BoneIndex.GetInt();

							if (ensureMsgf(ComponentSpaceMask[ParentBoneIdx] == true, TEXT("BoneMask::FCompactPoseRotationsCS. ComponentSpaceMask flag for Bone Index '%d' is not true"),
								ParentBoneIdx))
							{
								FQuat LocalPoseRotation = LocalPose[BoneIndex].GetRotation();
#if !(UE_BUILD_SHIPPING)
								if (LocalPoseRotation.ContainsNaN())
								{
									logOrEnsureNanError(TEXT("LocalPose for BoneIndex '%d' ContainsNaN"), BoneIdx);
									LocalPoseRotation = FQuat::Identity;
								}

								if (BoneRotationsCS[ParentBoneIdx].ContainsNaN())
								{
									logOrEnsureNanError(TEXT("BoneRotationsCS for BoneIndex '%d' ContainsNaN"), ParentBoneIdx);
									BoneRotationsCS[ParentBoneIdx] = FQuat::Identity;
								}
#endif

								// Use component space parent, and local space current bone
								BoneRotationsCS[BoneIdx] = BoneRotationsCS[ParentBoneIdx] * LocalPoseRotation;
								// Long chains might result in precision loss, so we need to normalize?
								BoneRotationsCS[BoneIdx].Normalize();

#if !(UE_BUILD_SHIPPING)
								if (BoneRotationsCS[BoneIdx].ContainsNaN())
								{
									logOrEnsureNanError(TEXT("BoneRotationsCS for BoneIndex '%d' ContainsNaN"), BoneIdx);
									BoneRotationsCS[BoneIdx] = FQuat::Identity;
								}
#endif

								ComponentSpaceMask[BoneIdx] = true;
							}
						}
					}
				}
			}
		}

	};

	struct FBoneMaskBlendScratchArea : public TThreadSingleton<FBoneMaskBlendScratchArea>
	{
		using FBoneMaskPoseCS = UE::Anim::BoneMask::FCompactPoseRotationsCS<FDefaultAllocator>;
		FBoneMaskPoseCS SourcePoseCS;
		// We usually only ever need 1 target pose, but allow more just in case.
		TArray<FBoneMaskPoseCS> TargetPosesCS;
		FBoneMaskPoseCS BlendPoseCS;

		TArray<float> MaxPoseWeights;
		TArray<const FBlendedCurve*> SourceCurves;
		TArray<float> SourceWeights;
	};

#if WITH_EDITORONLY_DATA
	TAutoConsoleVariable<int32> CVarBoneMaskWeightsForceRebuild(TEXT("a.AnimNode.Bonemask.WeightsForceRebuild"), 0, TEXT(""));
#endif

	static bool bUpdateCacheDataWhenBoneContainerChanges = true;
	static FAutoConsoleVariableRef CVarUpdateCacheDataWhenBoneContainerChanges(
		TEXT("a.AnimNode.Bonemask.UpdateCacheDataWhenBoneContainerChanges"), bUpdateCacheDataWhenBoneContainerChanges,
		TEXT("When true, cache data used by AnimNode_Layering will be re-cache everytime the bone container changes instead of only when the skeleton changes"),
		ECVF_Default);
}

static void BoneMaskBlend(FCompactPose& BasePose, TConstArrayView<FCompactPose> BlendPoses, FBlendedCurve& BaseCurve, TConstArrayView<FBlendedCurve> BlendedCurves,
	UE::Anim::FStackAttributeContainer& BaseAttributes, TConstArrayView<UE::Anim::FStackAttributeContainer> BlendAttributes, FAnimationPoseData& OutAnimationPoseData, 
	TConstArrayView<FPerBoneBlendWeight> BoneBlendWeightsForAttributes, TConstArrayView<UE::Anim::BoneMask::FBodyPartData> BodyPartsData, enum ECurveBlendOption::Type CurveBlendOption, bool bRootSpace = false)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BoneMaskBlend);

	//@TODO: Add support for mesh space scale blend?

	FCompactPose& OutPose = OutAnimationPoseData.GetPose();
	FBlendedCurve& OutCurve = OutAnimationPoseData.GetCurve();
	UE::Anim::FStackAttributeContainer& OutAttributes = OutAnimationPoseData.GetAttributes();

	// if no blendpose, outpose = basepose
	const int32 NumPoses = BlendPoses.Num();
	if (NumPoses == 0 || BodyPartsData.Num() == 0)
	{
		OutPose.MoveBonesFrom(BasePose);
		OutAttributes.MoveFrom(BaseAttributes);
		return;
	}

	const int32 NumBones = BasePose.GetNumBones();
	check(BoneBlendWeightsForAttributes.Num() == NumBones);
	check(OutPose.GetNumBones() == NumBones);

	for (const FPerBoneBlendWeight& PerBoneBlendWeight : BoneBlendWeightsForAttributes)
	{
		check(PerBoneBlendWeight.SourceIndex >= 0);
		check(PerBoneBlendWeight.SourceIndex < NumPoses);
	}

	for (const FCompactPose& BlendPose : BlendPoses)
	{
		check(BlendPose.GetNumBones() == NumBones);
	}

	using FBoneMaskBlendScratchArea = UE::Anim::BoneMask::FBoneMaskBlendScratchArea;
	FBoneMaskBlendScratchArea& ScratchArea = FBoneMaskBlendScratchArea::Get();
	TArray<float>& MaxPoseWeights = ScratchArea.MaxPoseWeights;
	MaxPoseWeights.Reset();
	MaxPoseWeights.AddZeroed(NumPoses);

	const FBoneContainer& BoneContainer = BasePose.GetBoneContainer();
	{
		{
			FBoneMaskBlendScratchArea::FBoneMaskPoseCS& SourcePoseCS = ScratchArea.SourcePoseCS;
			TArray<FBoneMaskBlendScratchArea::FBoneMaskPoseCS>& TargetPosesCS = ScratchArea.TargetPosesCS;
			FBoneMaskBlendScratchArea::FBoneMaskPoseCS& BlendPoseCS = ScratchArea.BlendPoseCS;
			// Every bone that isn't affected by a bone mask should use the local-space base as result.
			// @TODO: We're copying the whole pose here. Only copy the bones that aren't affected by ANY bone mask.
			{
				OutPose = BasePose;
			}

			{
				const int32 NumBlendPoses = BlendPoses.Num();
				TargetPosesCS.SetNum(NumBlendPoses, EAllowShrinking::No);
				for (int32 PoseIndex = 0; PoseIndex < NumBlendPoses; ++PoseIndex)
				{
					// Our target poses never change, so we can initialize only once.
					TargetPosesCS[PoseIndex].InitPose(BlendPoses[PoseIndex], bRootSpace);
				}
			}

			for (const UE::Anim::BoneMask::FBodyPartData& BodyPart : BodyPartsData)
			{
				const int32 BodyPartIndex = BodyPart.PartIndex;
				if (FAnimWeight::IsRelevant(BodyPart.LocalSpaceWeight))
				{
					// Only set the transforms of the bones affected by our body part.
					for(const UE::Anim::BoneMask::FPerBoneData& BoneData : BodyPart.PerBoneData)
					{
						const FCompactPoseBoneIndex BoneIndex = BoneData.BoneIndex;

						if (ensureMsgf(BasePose.IsValidIndex(BoneIndex), TEXT("BoneMask. Bone Index '%d' is not a valid index in BasePose ('%d' bones)"),
							BoneIndex.GetInt(), BasePose.GetNumBones()))
						{
							const int32 PoseIndex = BodyPart.SourcePoseIndex;
							const FCompactPoseBoneIndex ParentIndex = BoneContainer.GetParentBoneIndex(BoneIndex);

							const float BlendWeight = BoneData.Weight * BodyPart.LocalSpaceWeight;
							MaxPoseWeights[PoseIndex] = FMath::Max(MaxPoseWeights[PoseIndex], BlendWeight);

							const FTransform& BaseAtom = BasePose[BoneIndex];
							const FTransform& TargetAtom = BlendPoses[PoseIndex][BoneIndex];
							FTransform BlendAtom;

							if (!FAnimWeight::IsRelevant(BlendWeight))
							{
								BlendAtom = BaseAtom;
							}
							else if (FAnimWeight::IsFullWeight(BlendWeight))
							{
								BlendAtom = TargetAtom;
							}
							else
							{
								BlendAtom = BaseAtom;
								BlendAtom.BlendWith(TargetAtom, BlendWeight);
							}

							OutPose[BoneIndex] = BlendAtom;
							// Copy the result into the base pose so that the result can feed into future blends.
							BasePose[BoneIndex] = BlendAtom;
						}
					}
				}

				if (FAnimWeight::IsRelevant(BodyPart.MeshSpaceWeight))
				{
					// Source and blended component space poses need to be re-initialized to the result of the previous local-space blend.
					// @TODO: We probably don't need to reinitialize the whole chain, and only the bones that were actually affected.
					SourcePoseCS.InitPose(BasePose, bRootSpace);
					BlendPoseCS.InitPose(BasePose, bRootSpace);

					// Only set the transforms of the bones affected by our body part. 
					// Since this is a mesh space blend, we also need to fix-up the direct child bones of our body part. We'll do this later.
					for(const UE::Anim::BoneMask::FPerBoneData& BoneData : BodyPart.PerBoneData)
					{
						const FCompactPoseBoneIndex BoneIndex = BoneData.BoneIndex;

						if (ensureMsgf(BasePose.IsValidIndex(BoneIndex), TEXT("BoneMask Mesh Space Blend. Bone Index '%d' is not a valid index in BasePose ('%d' bones)"),
							BoneIndex.GetInt(), BasePose.GetNumBones()))
						{
							const int32 PoseIndex = BodyPart.SourcePoseIndex;
							check(BlendPoses.IsValidIndex(PoseIndex));
							const FCompactPoseBoneIndex ParentIndex = BoneContainer.GetParentBoneIndex(BoneIndex);

							const float BlendWeight = BoneData.Weight * BodyPart.MeshSpaceWeight;
							MaxPoseWeights[PoseIndex] = FMath::Max(MaxPoseWeights[PoseIndex], BlendWeight);

							const FTransform& BaseAtom = BasePose[BoneIndex];
							const FTransform& TargetAtom = BlendPoses[PoseIndex][BoneIndex];
							FTransform BlendAtom;

							if (!FAnimWeight::IsRelevant(BlendWeight))
							{
								BlendAtom = BaseAtom;
								BlendPoseCS.SetBoneRotationCS(BoneIndex, SourcePoseCS.SafeGetBoneRotationCS(BoneIndex, BasePose));
							}
							else if (FAnimWeight::IsFullWeight(BlendWeight))
							{
								BlendAtom = TargetAtom;
								BlendPoseCS.SetBoneRotationCS(BoneIndex, TargetPosesCS[PoseIndex].SafeGetBoneRotationCS(BoneIndex, BlendPoses[PoseIndex]));
							}
							else
							{
								BlendAtom = BaseAtom;
								BlendAtom.BlendWith(TargetAtom, BlendWeight);

								// Fast lerp produces un-normalized quaternions, so we'll re-normalize.
								const FQuat BlendedRotation = FQuat::FastLerp(SourcePoseCS.SafeGetBoneRotationCS(BoneIndex, BasePose), TargetPosesCS[PoseIndex].SafeGetBoneRotationCS(BoneIndex, BlendPoses[PoseIndex]), BlendWeight);
								BlendPoseCS.SetBoneRotationCS(BoneIndex, BlendedRotation.GetNormalized());
							}

							if (ParentIndex != INDEX_NONE)
							{
								// local -> mesh -> local transformations can cause loss of precision for long bone chains, we have to normalize rotation there.
								FQuat LocalBlendQuat = BlendPoseCS.SafeGetBoneRotationCS(ParentIndex, BasePose).Inverse() * BlendPoseCS.UnsafeGetBoneRotationCS(BoneIndex);
								LocalBlendQuat.Normalize();
								BlendAtom.SetRotation(LocalBlendQuat);
							}

							OutPose[BoneIndex] = BlendAtom;
						}
					}

					// Set the direct children of our body part to use the base pose mesh space rotations.
					for(const FCompactPoseBoneIndex& BoneIndex : BodyPart.ChildIndices)
					{						
						const FTransform& BaseAtom = BasePose[BoneIndex];
						const FCompactPoseBoneIndex ParentIndex = BoneContainer.GetParentBoneIndex(BoneIndex);
						FTransform BlendAtom;

						BlendAtom = BaseAtom;
						BlendPoseCS.SetBoneRotationCS(BoneIndex, SourcePoseCS.SafeGetBoneRotationCS(BoneIndex, BasePose));

						if (ParentIndex != INDEX_NONE)
						{
							// local -> mesh -> local transformations can cause loss of precision for long bone chains, we have to normalize rotation there.
							FQuat LocalBlendQuat = BlendPoseCS.SafeGetBoneRotationCS(ParentIndex, BasePose).Inverse() * BlendPoseCS.UnsafeGetBoneRotationCS(BoneIndex);
							LocalBlendQuat.Normalize();
							BlendAtom.SetRotation(LocalBlendQuat);
						}

						OutPose[BoneIndex] = BlendAtom;
					}

					// Apply any changes to the base pose.
					// @TODO: We're copying the whole pose here. We should only copy the bones that were affected by the mesh space body part (including direct children).
					BasePose = OutPose;
				}
			}
		}
	}

	// time to blend curves
	// the way we blend curve per bone
	// is to find out max weight per that pose, and then apply that weight to the curve
	{
		TArray<const FBlendedCurve*>& SourceCurves = ScratchArea.SourceCurves;
		TArray<float>& SourceWeights = ScratchArea.SourceWeights;

		SourceCurves.Reset();
		SourceCurves.SetNumUninitialized(NumPoses + 1);
		SourceWeights.Reset();
		SourceWeights.SetNumUninitialized(NumPoses + 1);

		SourceCurves[0] = &BaseCurve;
		SourceWeights[0] = 1.f;

		for (int32 Idx = 0; Idx < NumPoses; ++Idx)
		{
			SourceCurves[Idx + 1] = &BlendedCurves[Idx];
			SourceWeights[Idx + 1] = MaxPoseWeights[Idx];
		}

		//@TODO: Temporally copy-pasting the BlendCurves implementation (from AnimationRuntime.cpp) until we expose it to the outside

		//BlendCurves(SourceCurves, SourceWeights, OutCurve, CurveBlendOption);
		if (SourceCurves.Num() > 0)
		{
			ECurveBlendOption::Type BlendOption = CurveBlendOption;
			if (BlendOption == ECurveBlendOption::Type::BlendByWeight)
			{
				BlendCurves(SourceCurves, SourceWeights, OutCurve);
			}
			else if (BlendOption == ECurveBlendOption::Type::NormalizeByWeight)
			{
				float SumOfWeight = 0.f;
				for (const auto& Weight : SourceWeights)
				{
					SumOfWeight += Weight;
				}

				if (FAnimWeight::IsRelevant(SumOfWeight))
				{
					TArray<float> NormalizeSourceWeights;
					NormalizeSourceWeights.AddUninitialized(SourceWeights.Num());
					for (int32 Idx = 0; Idx < SourceWeights.Num(); ++Idx)
					{
						NormalizeSourceWeights[Idx] = SourceWeights[Idx] / SumOfWeight;
					}

					BlendCurves(SourceCurves, NormalizeSourceWeights, OutCurve);
				}
				else
				{
					BlendCurves(SourceCurves, SourceWeights, OutCurve);
				}
			}
			else if (BlendOption == ECurveBlendOption::Type::UseMaxValue)
			{
				OutCurve.Override(*SourceCurves[0], SourceWeights[0]);

				for (int32 CurveIndex = 1; CurveIndex < SourceCurves.Num(); ++CurveIndex)
				{
					OutCurve.UseMaxValue(*SourceCurves[CurveIndex]);
				}
			}
			else if (BlendOption == ECurveBlendOption::Type::UseMinValue)
			{
				OutCurve.Override(*SourceCurves[0], SourceWeights[0]);

				for (int32 CurveIndex = 1; CurveIndex < SourceCurves.Num(); ++CurveIndex)
				{
					OutCurve.UseMinValue(*SourceCurves[CurveIndex]);
				}
			}
			else if (BlendOption == ECurveBlendOption::Type::UseBasePose)
			{
				OutCurve.Override(*SourceCurves[0], SourceWeights[0]);
			}
			else if (BlendOption == ECurveBlendOption::Type::DoNotOverride)
			{
				OutCurve.Override(*SourceCurves[0], SourceWeights[0]);

				for (int32 CurveIndex = 1; CurveIndex < SourceCurves.Num(); ++CurveIndex)
				{
					OutCurve.CombinePreserved(*SourceCurves[CurveIndex]);
				}
			}
			else
			{
				OutCurve.Override(*SourceCurves[0], SourceWeights[0]);

				for (int32 CurveIndex = 1; CurveIndex < SourceCurves.Num(); ++CurveIndex)
				{
					OutCurve.Combine(*SourceCurves[CurveIndex]);
				}
			}
		}
	}

	{
		// @TODO: Custom attributes need to blend iteratively too, but pass through for now
		//UE::Anim::Attributes::BlendAttributesPerBoneFilter(BaseAttributes, BlendAttributes, BoneBlendWeightsForAttributes, OutAttributes);
		OutAttributes.MoveFrom(BaseAttributes);
	}
}

/////////////////////////////////////////////////////
// FAnimNode_BoneMask

void FAnimNode_BoneMask::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	const int NumPoses = BlendPoses.Num();
	checkSlow(BlendWeights.Num() == NumPoses);

	// initialize children
	BasePose.Initialize(Context);

	if (NumPoses > 0)
	{
		for (int32 ChildIndex = 0; ChildIndex < NumPoses; ++ChildIndex)
		{
			BlendPoses[ChildIndex].Initialize(Context);
		}
	}
}

void FAnimNode_BoneMask::UpdateCachedBoneData(const FBoneContainer& RequiredBones, const USkeleton* Skeleton)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_BoneMask_UpdateCachedBoneData);

	bool bForceRebuild = false;
#if WITH_EDITOR
	bForceRebuild = UE::Anim::BoneMask::CVarBoneMaskWeightsForceRebuild.GetValueOnAnyThread() > 0;
#endif

	if (BoneMaskDefinitionDataAsset == nullptr || Skeleton == nullptr)
	{
		return;
	}

	// @TODO: We could probably cache the compact pose, and only update per-part weight changes.
	// Might need some extra work to work across LOD swaps.
	if (bForceRebuild || (RequiredBones.GetSerialNumber() != RequiredBonesSerialNumber))
	{
	}

	const FBoneMaskDefinition& BoneMaskDefinition = BoneMaskDefinitionDataAsset->GetBoneMaskDefinition();

	if (bUpdateSkeletonDataOnDemand)
	{
		if (((UE::Anim::BoneMask::bUpdateCacheDataWhenBoneContainerChanges == true) && (RequiredBones.GetSerialNumber() != RequiredBonesSerialNumber)) ||
			((UE::Anim::BoneMask::bUpdateCacheDataWhenBoneContainerChanges == false) && (CachedSkeleton.Get() != Skeleton)))
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_BoneMask_UpdateSkeletonDataOnDemand);

			auto GetBoneWeightsForBodyPartToFill = [&, this](int32 BodyPartIdx) -> TArray<FBoneMaskPerBoneData>&{ return BodyPartsSkeletonData[BodyPartIdx].SkeletonPoseBoneWeights; };

			auto GetChildBoneIndicesForBodyPartToFill = [&, this](int32 BodyPartIdx) -> TArray<int32>&{ return BodyPartsSkeletonData[BodyPartIdx].SkeletonPoseChildBoneIndices; };

			BodyPartsSkeletonData.Reset(BoneMaskDefinition.BodyPartDefinitions.Num());
			BodyPartsSkeletonData.AddDefaulted(BoneMaskDefinition.BodyPartDefinitions.Num());
			BoneMaskDefinitionDataAsset->UpdateBoneMaskCachedData(*Skeleton, BoneMaskDefinition, GetBoneWeightsForBodyPartToFill, GetChildBoneIndicesForBodyPartToFill);

			CachedSkeleton = Skeleton;
		}
	}

	auto GetBoneWeightsForBodyPartConst = [&, this](int32 BodyPartIdx) -> const TArray<FBoneMaskPerBoneData>&
		{
			return bUpdateSkeletonDataOnDemand ? BodyPartsSkeletonData[BodyPartIdx].SkeletonPoseBoneWeights : BoneMaskDefinition.BodyPartDefinitions[BodyPartIdx].SkeletonPoseBoneWeights;
		};

	auto GetChildBoneIndicesForBodyPartConst = [&, this](int32 BodyPartIdx) -> const TArray<int32>&
		{
			return bUpdateSkeletonDataOnDemand ? BodyPartsSkeletonData[BodyPartIdx].SkeletonPoseChildBoneIndices : BoneMaskDefinition.BodyPartDefinitions[BodyPartIdx].SkeletonPoseChildBoneIndices;
		};

	// build desired bone weights
	const TArray<FBoneIndexType>& RequiredBoneIndices = RequiredBones.GetBoneIndicesArray();
	const int32 NumRequiredBones = RequiredBoneIndices.Num();

	// We must build a compact pose bone weights list for custom attributes/linked bones.
	// @TODO: Optionally skip this.
	CurrentBoneBlendWeightsForAttributes.Reset(NumRequiredBones);
	CurrentBoneBlendWeightsForAttributes.AddZeroed(NumRequiredBones);

	//Reinitialize bone blend weights now that we have cleared them
	{
		const int32 BodyPartsNum = BoneMaskDefinition.BodyPartDefinitions.Num();
		BodyPartsData.SetNum(BodyPartsNum, EAllowShrinking::No);
		for (int32 BodyPartIndex = 0; BodyPartIndex < BodyPartsNum; ++BodyPartIndex)
		{
			const TArray<FBoneMaskPerBoneData>& SkeletonPoseBoneWeights = GetBoneWeightsForBodyPartConst(BodyPartIndex);
			const TArray<int32>& SkeletonPoseChildBoneIndices = GetChildBoneIndicesForBodyPartConst(BodyPartIndex);

			UE::Anim::BoneMask::FBodyPartData& BodyPart = BodyPartsData[BodyPartIndex];
			const FBoneMaskBodyPartDefinition& BodyPartDef = BoneMaskDefinition.BodyPartDefinitions[BodyPartIndex];
			FBoneMaskEntry BoneMaskEntry = BoneMask.BoneMaskMap.FindRef(BodyPartDef.Name, FBoneMaskEntry());
			BodyPart.LocalSpaceWeight = BoneMaskEntry.LocalSpaceWeight;
			BodyPart.MeshSpaceWeight = BoneMaskEntry.MeshSpaceWeight;

			float PartWeight = FMath::Max(BodyPart.MeshSpaceWeight, BodyPart.LocalSpaceWeight);
			if (!FAnimWeight::IsRelevant(PartWeight))
			{
				// If the body part has 0 weight, we can skip it.
				continue;
			}

#if WITH_EDITORONLY_DATA
			BodyPart.DebugPartName = BodyPartDef.Name;
#endif

			// Find the Pose for each body part
			// BodyParts indices are mapped to BlendPoses indices but starting at the second index, 
			// the first pose in BlendPoses is reserved for any body part in the mask that is not explicitly added to BodyParts array
			// E.g The first body part(s) in this array will use the second pose in BlendPoses. Assuming there are no more body parts in this array, all the other parts will use the first pose.
			int32 PoseIndex = 0;

			const FName BodyPartName = BodyPartDef.Name;
			for (int32 Idx = 0; Idx < BodyParts.Num(); Idx++)
			{
				if (BodyParts[Idx].Names.ContainsByPredicate([&BodyPartName](const FName& Item) { return Item == BodyPartName; }))
				{
					PoseIndex = Idx + 1;
					break;
				}
			}
			BodyPart.SourcePoseIndex = PoseIndex;

			const int32 NumBonesInBodyPart = SkeletonPoseBoneWeights.Num();
			BodyPart.PartIndex = BodyPartIndex;
			BodyPart.PerBoneData.Reset(NumBonesInBodyPart);
			for (const FBoneMaskPerBoneData& BoneData : SkeletonPoseBoneWeights)
			{
				const FCompactPoseBoneIndex CompactPoseBoneIndex = RequiredBones.GetCompactPoseIndexFromSkeletonPoseIndex(FSkeletonPoseBoneIndex(BoneData.SkeletonPoseBoneIndex));
				if (CompactPoseBoneIndex != INDEX_NONE)
				{
					BodyPart.PerBoneData.Add(UE::Anim::BoneMask::FPerBoneData(CompactPoseBoneIndex, BoneData.BlendWeight));

					CurrentBoneBlendWeightsForAttributes[CompactPoseBoneIndex.GetInt()].SourceIndex = PoseIndex;
					CurrentBoneBlendWeightsForAttributes[CompactPoseBoneIndex.GetInt()].BlendWeight = PartWeight * BoneData.BlendWeight;
				}
			}

			const int32 NumBonesInChildrenOfBodyPart = SkeletonPoseChildBoneIndices.Num();
			BodyPart.ChildIndices.Reset(NumBonesInChildrenOfBodyPart);
			for (const int32 ChildBoneIndex : SkeletonPoseChildBoneIndices)
			{
				const FCompactPoseBoneIndex CompactPoseBoneIndex = RequiredBones.GetCompactPoseIndexFromSkeletonPoseIndex(FSkeletonPoseBoneIndex(ChildBoneIndex));
				if (CompactPoseBoneIndex != INDEX_NONE)
				{
					BodyPart.ChildIndices.Emplace(CompactPoseBoneIndex);
				}
			}
		}
	}

	// Build curve source indices
	{
		// Get the original Reserve value
		int32 OriginalReserve = CurvePoseSourceIndices.Max();
		CurvePoseSourceIndices.Empty();

		Skeleton->ForEachCurveMetaData([this, &RequiredBones](const FName& InCurveName, const FCurveMetaData& InMetaData)
		{
			for (const FBoneReference& LinkedBone : InMetaData.LinkedBones)
			{
				FCompactPoseBoneIndex CompactPoseIndex = LinkedBone.GetCompactPoseIndex(RequiredBones);
				if (CompactPoseIndex != INDEX_NONE)
				{
					if (CurrentBoneBlendWeightsForAttributes[CompactPoseIndex.GetInt()].BlendWeight > 0.f)
					{
						CurvePoseSourceIndices.Add(InCurveName, CurrentBoneBlendWeightsForAttributes[CompactPoseIndex.GetInt()].SourceIndex);
						break;
					}
				}
			}
		});

		// Shrink afterwards to exactly what was used if the Reserve increased, to save memory.  Eventually the reserve will
		// stabilize at the maximum number of nodes actually used in practice for this specific anim node.
		if (CurvePoseSourceIndices.Num() > OriginalReserve)
		{
			CurvePoseSourceIndices.Shrink();
		}
	}

	RequiredBonesSerialNumber = RequiredBones.GetSerialNumber();
}

void FAnimNode_BoneMask::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)

	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_BoneMask_CacheBones);

	BasePose.CacheBones(Context);
	int32 NumPoses = BlendPoses.Num();
	for (int32 ChildIndex = 0; ChildIndex < NumPoses; ChildIndex++)
	{
		BlendPoses[ChildIndex].CacheBones(Context);
	}

	const FBoneContainer& RequiredBones = Context.AnimInstanceProxy->GetRequiredBones();
	if (RequiredBones.GetSerialNumber() != RequiredBonesSerialNumber)
	{
		// This shouldn't be necessary because the update should already have built the bone mask for this frame,
		// but we have an edge case where we're calling Evaluation without having called the Update.
		UpdateCachedBoneData(Context.AnimInstanceProxy->GetRequiredBones(), Context.AnimInstanceProxy->GetSkeleton());
	}
}

void FAnimNode_BoneMask::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)

	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_BoneMask_Update);

	bHasRelevantPoses = false;
	int32 RootMotionBlendPose = -1;
	float RootMotionWeight = 0.f;

	if (IsLODEnabled(Context.AnimInstanceProxy))
	{
		GetEvaluateGraphExposedInputs().Execute(Context);

		if  (BoneMaskDefinitionDataAsset)
		{
			for (int32 ChildIndex = 0; ChildIndex < BlendPoses.Num(); ++ChildIndex)
			{
				const float ChildWeight = BlendWeights[ChildIndex];
				if (FAnimWeight::IsRelevant(ChildWeight))
				{
					if (bHasRelevantPoses == false)
					{
						UpdateCachedBoneData(Context.AnimInstanceProxy->GetRequiredBones(), Context.AnimInstanceProxy->GetSkeleton());

						bHasRelevantPoses = true;
					}

					// We're never layering in the root bone, so keep root motion at 0 weight.
					BlendPoses[ChildIndex].Update(Context.FractionalWeightAndRootMotion(ChildWeight, 0.0f));
				}
			}
		}
	}

	// initialize children
	const float BaseRootMotionWeight = 1.f - RootMotionWeight;
	if (BaseRootMotionWeight < ZERO_ANIMWEIGHT_THRESH)
	{
		BasePose.Update(Context.FractionalWeightAndRootMotion(1.f, BaseRootMotionWeight));
	}
	else
	{
		BasePose.Update(Context);
	}

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Num Poses"), BlendPoses.Num());
}

void FAnimNode_BoneMask::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_BoneMask_Evaluate);
	
	using namespace UE::Anim;

	const int NumPoses = BlendPoses.Num();
	if ((NumPoses == 0) || !bHasRelevantPoses || (BodyPartsData.Num() == 0) || (BoneMaskDefinitionDataAsset == nullptr))
	{
		BasePose.Evaluate(Output);
	}
	else
	{
		FPoseContext BasePoseContext(Output);

		// evaluate children
		BasePose.Evaluate(BasePoseContext);

		TArrayView<FCompactPose> TargetBlendPoses((FCompactPose*)FMemory_Alloca(NumPoses * sizeof(FCompactPose)), NumPoses);
		TArrayView<FBlendedCurve> TargetBlendCurves((FBlendedCurve*)FMemory_Alloca(NumPoses * sizeof(FBlendedCurve)), NumPoses);
		TArrayView<FStackAttributeContainer> TargetBlendAttributes((FStackAttributeContainer*)FMemory_Alloca(NumPoses * sizeof(FStackAttributeContainer)), NumPoses);

		for (int32 ChildIndex = 0; ChildIndex < NumPoses; ++ChildIndex)
		{
			// Call constructors manually since we used alloca
			new (&TargetBlendPoses[ChildIndex]) FCompactPose;
			new (&TargetBlendCurves[ChildIndex]) FBlendedCurve;
			new (&TargetBlendAttributes[ChildIndex]) FStackAttributeContainer;

			if (FAnimWeight::IsRelevant(BlendWeights[ChildIndex]))
			{
				FPoseContext CurrentPoseContext(Output);
				BlendPoses[ChildIndex].Evaluate(CurrentPoseContext);

				TargetBlendPoses[ChildIndex].MoveBonesFrom(CurrentPoseContext.Pose);
				TargetBlendCurves[ChildIndex].MoveFrom(CurrentPoseContext.Curve);
				TargetBlendAttributes[ChildIndex].MoveFrom(CurrentPoseContext.CustomAttributes);
			}
			else
			{
				TargetBlendPoses[ChildIndex].ResetToRefPose(BasePoseContext.Pose.GetBoneContainer());
				TargetBlendCurves[ChildIndex].InitFrom(Output.Curve);
			}
		}

		// filter to make sure it only includes curves that are linked to the correct bone filter
		FNamedValueArrayUtils::RemoveByPredicate(BasePoseContext.Curve, CurvePoseSourceIndices,
			[](const UE::Anim::FCurveElement& InOutBasePoseElement, const UE::Anim::FCurveElementIndexed& InSourceIndexElement)
			{
				// if source index is set, remove base pose curve value
				return (InSourceIndexElement.Index != INDEX_NONE);
			});

		// Filter child pose curves
		for (int32 ChildIndex = 0; ChildIndex < NumPoses; ++ChildIndex)
		{
			FNamedValueArrayUtils::RemoveByPredicate(TargetBlendCurves[ChildIndex], CurvePoseSourceIndices,
				[ChildIndex](const UE::Anim::FCurveElement& InOutBasePoseElement, const UE::Anim::FCurveElementIndexed& InSourceIndexElement)
				{
					// if not source, remove it
					return (InSourceIndexElement.Index != INDEX_NONE) && (InSourceIndexElement.Index != ChildIndex);
				});
		}

		FAnimationPoseData AnimationPoseData(Output);
		BoneMaskBlend(BasePoseContext.Pose, TargetBlendPoses, BasePoseContext.Curve, TargetBlendCurves, BasePoseContext.CustomAttributes, TargetBlendAttributes, AnimationPoseData, CurrentBoneBlendWeightsForAttributes, BodyPartsData, CurveBlendOption, bProcessMeshSpaceAsRootSpace);

		// Call destructors manually since we used alloca
		for (int32 ChildIndex = 0; ChildIndex < NumPoses; ++ChildIndex)
		{
			TargetBlendPoses[ChildIndex].~FCompactPose();
			TargetBlendCurves[ChildIndex].~FBlendedCurve();
			TargetBlendAttributes[ChildIndex].~FStackAttributeContainer();
		}
	}
}


void FAnimNode_BoneMask::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	const int NumPoses = BlendPoses.Num();

	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Num Poses: %i)"), NumPoses);
	DebugData.AddDebugItem(DebugLine);

	BasePose.GatherDebugData(DebugData.BranchFlow(1.f));

	for (int32 ChildIndex = 0; ChildIndex < NumPoses; ++ChildIndex)
	{
		BlendPoses[ChildIndex].GatherDebugData(DebugData.BranchFlow(BlendWeights[ChildIndex]));
	}
}