// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_CopyBoneAdvanced.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_CopyBoneAdvanced)

/////////////////////////////////////////////////////
// FAnimNode_CopyBoneAdvanced

DECLARE_CYCLE_STAT(TEXT("CopyBoneAdvanced Eval"), STAT_CopyBoneAdvanced_Eval, STATGROUP_Anim);

FAnimNode_CopyBoneAdvanced::FAnimNode_CopyBoneAdvanced()
	: TranslationWeight(FVector::One())
	, RotationWeight(1.0f)
	, ScaleWeight(1.0f)
	, ControlSpace(BCS_ComponentSpace)
	, bTranslationInCustomBoneSpace(true)
	, bPropagateToChildren(true)
{
}

void FAnimNode_CopyBoneAdvanced::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	
	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += FString::Printf(TEXT(" Src: %s Dst: %s)"), *SourceBone.BoneName.ToString(), *TargetBone.BoneName.ToString());
	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_CopyBoneAdvanced::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	SCOPE_CYCLE_COUNTER(STAT_CopyBoneAdvanced_Eval);
	check(OutBoneTransforms.Num() == 0);

	// Pass through if we're not doing anything.
	const bool bCopyTranslation = !TranslationWeight.IsNearlyZero();
	const bool bCopyRotation = !FMath::IsNearlyZero(RotationWeight);
	const bool bCopyScale = !FMath::IsNearlyZero(ScaleWeight);
	if(!bCopyTranslation && !bCopyRotation && !bCopyScale)
	{
		return;
	}

	// Get component space transform for source and target bone.
	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
	FCompactPoseBoneIndex SourceBoneIndex = SourceBone.GetCompactPoseIndex(BoneContainer);
	FCompactPoseBoneIndex TargetBoneIndex = TargetBone.GetCompactPoseIndex(BoneContainer);

	FTransform SourceBoneTM = Output.Pose.GetComponentSpaceTransform(SourceBoneIndex);
	FTransform TargetBoneTM = Output.Pose.GetComponentSpaceTransform(TargetBoneIndex);

	// Our translation axes start in component space
	FMatrix TranslationWeightMatrix(FVector::ForwardVector, FVector::RightVector, FVector::UpVector, FVector::Zero());
	if (bTranslationInCustomBoneSpace)
	{
		FTransform TranslationBoneSpaceTM;
		if (TranslationSpaceBone.BoneIndex == INDEX_NONE)
		{
			// No bone specified. Use the source bone as our reference frame
			TranslationBoneSpaceTM = TargetBoneTM;
		}
		else
		{
			FCompactPoseBoneIndex TranslationBoneIndex = TranslationSpaceBone.GetCompactPoseIndex(BoneContainer);
			TranslationBoneSpaceTM = Output.Pose.GetComponentSpaceTransform(TranslationBoneIndex);
		}
		// Convert our reference frame to the specified control space.
		FAnimationRuntime::ConvertCSTransformToBoneSpace(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, TranslationBoneSpaceTM, SourceBoneIndex, ControlSpace);

		// Make our translation axes relative to our reference frame
		const FQuat BoneSpaceQuat = TranslationBoneSpaceTM.GetRotation();
		TranslationWeightMatrix = BoneSpaceQuat.ToMatrix();
	}

	if(ControlSpace != BCS_ComponentSpace)
	{
		// Convert out to selected space
		FAnimationRuntime::ConvertCSTransformToBoneSpace(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, SourceBoneTM, SourceBoneIndex, ControlSpace);
		FAnimationRuntime::ConvertCSTransformToBoneSpace(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, TargetBoneTM, TargetBoneIndex, ControlSpace);
	}
	
	// Copy individual components
	if (bCopyTranslation)
	{
		// Apply a per-axis percentage of the translation delta to the target bone.
		const FVector DeltaTranslation = SourceBoneTM.GetTranslation() - TargetBoneTM.GetTranslation();

		FVector Forward, Right, Up;
		TranslationWeightMatrix.GetScaledAxes(Forward, Right, Up);

		// Apply a percentage of the translation delta in each axis, relative to our reference frame.
		FVector WeightedDelta = TranslationWeight.X * Forward * Forward.Dot(DeltaTranslation);
		WeightedDelta += TranslationWeight.Y * Right * Right.Dot(DeltaTranslation);
		WeightedDelta += TranslationWeight.Z * Up * Up.Dot(DeltaTranslation);

		TargetBoneTM.AddToTranslation(WeightedDelta);
	}

	if (bCopyRotation)
	{
		// Add a percentage of the rotation delta to the target bone
		TargetBoneTM.SetRotation(FQuat::Slerp(TargetBoneTM.GetRotation(), SourceBoneTM.GetRotation(), RotationWeight));
	}

	if (bCopyScale)
	{
		// Add a percentage of the scale delta to the target bone
		const FVector DeltaScale = SourceBoneTM.GetScale3D() - TargetBoneTM.GetScale3D();
		TargetBoneTM.SetScale3D(TargetBoneTM.GetScale3D() + DeltaScale * ScaleWeight);
	}

	if(ControlSpace != BCS_ComponentSpace)
	{
		// Convert back out if we aren't operating in component space
		FAnimationRuntime::ConvertBoneSpaceTransformToCS(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, TargetBoneTM, TargetBoneIndex, ControlSpace);
	}

	// Output new transform for current bone.
	if (bPropagateToChildren)
	{
		// Transforms are propagated to children in FCSPose<PoseType>::LocalBlendCSBoneTransforms.
		OutBoneTransforms.Add(FBoneTransform(TargetBoneIndex, TargetBoneTM));
	}
	else
	{
		// Calculate component space transform of every direct child, to conserve their component space transform.
		for (const FCompactPoseBoneIndex& ChildCompactIdx : ChildBoneIndices)
		{
			// GetComponentSpaceTransform will force ComponentSpaceFlags to be set.
			Output.Pose.GetComponentSpaceTransform(ChildCompactIdx);
		}

		// Set the component space transform directly without affecting any children.
		Output.Pose.SetComponentSpaceTransform(TargetBoneIndex, TargetBoneTM);
	}

	TRACE_ANIM_NODE_VALUE(Output, TEXT("Source Bone"), SourceBone.BoneName);
	TRACE_ANIM_NODE_VALUE(Output, TEXT("Target Bone"), TargetBone.BoneName);
}

bool FAnimNode_CopyBoneAdvanced::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) 
{
	if ((TranslationSpaceBone.BoneIndex != INDEX_NONE) && bTranslationInCustomBoneSpace)
	{
		if (TranslationSpaceBone.IsValidToEvaluate(RequiredBones) == false)
		{
			return false;
		}
	}

	return (TargetBone.IsValidToEvaluate(RequiredBones) && (TargetBone==SourceBone || SourceBone.IsValidToEvaluate(RequiredBones)));
}

void FAnimNode_CopyBoneAdvanced::InitializeBoneReferences(const FBoneContainer& RequiredBones) 
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	SourceBone.Initialize(RequiredBones);
	TargetBone.Initialize(RequiredBones);
	TranslationSpaceBone.Initialize(RequiredBones);

	if ((bPropagateToChildren == false) && TargetBone.IsValidToEvaluate(RequiredBones))
	{
		// If we're not propagating our transform to children, 
		// cache our  direct children indices, so we can calculate their component space transforms during eval.
		const FMeshPoseBoneIndex TargetBoneMeshPoseIdx = TargetBone.GetMeshPoseIndex(RequiredBones);
		TArray<int32> ChildBoneIndicesAsInt;
		RequiredBones.GetReferenceSkeleton().GetDirectChildBones(TargetBoneMeshPoseIdx.GetInt(), ChildBoneIndicesAsInt);
		ChildBoneIndices.Reserve(ChildBoneIndicesAsInt.Num());
		for (const int32 ChildIdx : ChildBoneIndicesAsInt)
		{
			const FMeshPoseBoneIndex MeshPoseBoneIdx = FMeshPoseBoneIndex(ChildIdx);
			const FSkeletonPoseBoneIndex SkeletonPoseBoneIndex = RequiredBones.GetSkeletonPoseIndexFromMeshPoseIndex(MeshPoseBoneIdx);
			const FCompactPoseBoneIndex CompactIndex = RequiredBones.GetCompactPoseIndexFromSkeletonPoseIndex(SkeletonPoseBoneIndex);

			if (CompactIndex != INDEX_NONE)
			{
				ChildBoneIndices.Add(CompactIndex);
			}
		}
	}
}
