// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_CopyBoneMotion.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimSubsystem_Tag.h"
#include "Math/SpringMath.h"
#include "Curves/CurveVector.h"
#include "Curves/CurveFloat.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/UObjectToken.h"
#include "PoseSearch/AnimNode_PoseSearchHistoryCollector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_CopyBoneMotion)

#define LOCTEXT_NAMESPACE "AnimNode_CopyBoneMotion"

DECLARE_CYCLE_STAT(TEXT("CopyBoneMotion Eval"), STAT_CopyMotion_Eval, STATGROUP_Anim);

namespace UE::Anim::CopyBoneMotion
{
#if ENABLE_ANIM_DEBUG
	static TAutoConsoleVariable<int32> CVarDebug(TEXT("a.AnimNode.CopyBoneMotion.Debug"), 0, TEXT("Turn on visualization debugging for Copy Bone Motion. 1: Debug draw coordinate systems, 2: Only show copied motion, 3: Draw motion history, 4: Enable all"));
#endif
	static TAutoConsoleVariable<int32> CVarEnable(TEXT("a.AnimNode.CopyBoneMotion.Enable"), 1, TEXT("Toggle the Copy Bone Motion node."));

	bool IsDebugEnabled()
	{
#if ENABLE_ANIM_DEBUG
		return CVarDebug.GetValueOnAnyThread() > 0;
#else
		return false;
#endif
	}

	bool ShouldShowCopiedMotion()
	{
#if ENABLE_ANIM_DEBUG
		return	CVarDebug.GetValueOnAnyThread() == 2 ||
				CVarDebug.GetValueOnAnyThread() == 4;
#else
		return false;
#endif
	}

	bool ShouldDrawMotionHistory()
	{
#if ENABLE_ANIM_DEBUG
		return	CVarDebug.GetValueOnAnyThread() == 3 ||
				CVarDebug.GetValueOnAnyThread() == 4;
#else
		return false;
#endif
	}

	FQuat ReOrientDeltaQuat(const FQuat DeltaQuat, const FQuat NewOrientation)
	{
		FVector DeltaAxis;
		float DeltaAngle;
		DeltaQuat.ToAxisAndAngle(DeltaAxis, DeltaAngle);

		FVector ReOrientedAxis = NewOrientation.RotateVector(DeltaAxis);
		//Calculate the new delta rotation
		FQuat ReOrientedQuat(ReOrientedAxis, DeltaAngle);
		ReOrientedQuat.Normalize();

		return ReOrientedQuat;
	}

	const bool GetPoseHistoryCollectorNodeFromTag(const FName Tag, const UAnimInstance* AnimInstance, const FAnimNode_PoseSearchHistoryCollector_Base*& OutNode)
	{
		if (IAnimClassInterface* AnimBlueprintClass = IAnimClassInterface::GetFromClass(AnimInstance->GetClass()))
		{
			if (const FAnimSubsystem_Tag* TagSubsystem = AnimBlueprintClass->FindSubsystem<FAnimSubsystem_Tag>())
			{
				OutNode = TagSubsystem->FindNodeByTag<const FAnimNode_PoseSearchHistoryCollector_Base>(Tag, AnimInstance);
				if (OutNode)
				{
					return true;
				}
			}
		}

		return false;
	}

	FVector GetAxisVector(const EAxis::Type InAxis)
	{
		switch (InAxis)
		{
		case EAxis::X: return FVector::ForwardVector;
		case EAxis::Y: return FVector::RightVector;
		default:
		case EAxis::Z: return FVector::UpVector;
		};
	}
	float GetSignedAngleAroundAxis(const FQuat& Rotation, const FVector& TwistAxis)
	{
		FQuat Swing, Twist;

		Rotation.ToSwingTwist(TwistAxis, Swing, Twist);
		if (FVector::DotProduct(Twist.GetRotationAxis(), TwistAxis) > 0)
		{
			return -Twist.GetAngle();
		}
		else
		{
			return Twist.GetAngle();
		}
	}

	FTransform GetTransformDelta(const FTransform& InFrom, const FTransform& InTo)
	{
		// Compute delta between 2 transforms in the same reference frame, ignoring scale as we don't use it for copy motion.
		FVector TranslationDelta = InTo.GetTranslation() - InFrom.GetTranslation();
		FQuat RotationDelta = InTo.GetRotation() * InFrom.GetRotation().Inverse();
		RotationDelta.Normalize();
		return FTransform(RotationDelta, TranslationDelta, FVector(1.f,1.f,1.f));
	}

	void RemoveRootRotation(FTransform& InOutTransform, const FQuat& RootRotInv)
	{
		InOutTransform.SetRotation(RootRotInv * InOutTransform.GetRotation());
		InOutTransform.SetTranslation(RootRotInv.RotateVector(InOutTransform.GetTranslation()));
	}
};

FAnimNode_CopyBoneMotion::FAnimNode_CopyBoneMotion()
{
}

void FAnimNode_CopyBoneMotion::GatherDebugData(FNodeDebugData& DebugData)
{
	Super::GatherDebugData(DebugData);

	if (bUseBasePose)
	{
		BasePose.GatherDebugData(DebugData);
		BasePoseReference.GatherDebugData(DebugData);
	}
}

void FAnimNode_CopyBoneMotion::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	Super::Initialize_AnyThread(Context);

	if (bUseBasePose)
	{
		BasePose.Initialize(Context);
		BasePoseReference.Initialize(Context);
	}
}

void FAnimNode_CopyBoneMotion::UpdateInternal(const FAnimationUpdateContext& Context)
{
	Super::UpdateInternal(Context);

	// If we just became relevant and haven't been initialized yet, then reinitialize foot placement.
	if (!bIsFirstUpdate && UpdateCounter.HasEverBeenUpdated() && !UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter()))
	{
		Reset();
	}
	UpdateCounter.SynchronizeWith(Context.AnimInstanceProxy->GetUpdateCounter());
}

void FAnimNode_CopyBoneMotion::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	SCOPE_CYCLE_COUNTER(STAT_CopyMotion_Eval);
	check(OutBoneTransforms.Num() == 0);
	Super::EvaluateSkeletalControl_AnyThread(Output, OutBoneTransforms);

	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
	FTransformNoScale TransformOffset;
	EBoneControlSpace TransformOffsetSpace = EBoneControlSpace::BCS_ComponentSpace;

	if (bUseBasePose)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_CopyMotion_CopyTransforms);

		FComponentSpacePoseContext BaseData(Output);
		{
			BasePose.EvaluateComponentSpace(BaseData);
		}

		FComponentSpacePoseContext ReferencePoseData(Output);
		{
			BasePoseReference.EvaluateComponentSpace(ReferencePoseData);
		}

		const FCompactPoseBoneIndex CompactPoseSourceBoneIdx = SourceBone.GetCompactPoseIndex(BoneContainer);

		FTransform SourceToTransform;
		const FAnimNode_PoseSearchHistoryCollector_Base* PoseHistoryNode = nullptr;
		const UE::PoseSearch::FPoseHistory* PoseHistory = nullptr;
		//@todo: Error out on missing, but expected pose history node. Can this be caught at compile time and cached?
		if (bUseBasePose && (Delay > 0.0f))
		{
			if (UE::Anim::CopyBoneMotion::GetPoseHistoryCollectorNodeFromTag(PoseHistoryTag, CastChecked<const UAnimInstance>(Output.GetAnimInstanceObject()), PoseHistoryNode))
			{
				PoseHistory = PoseHistoryNode->GetPoseHistoryPtr();
			}
		}

		// Helper to conditionally read base pose bone transforms from history
		const USkeleton* Skeleton = BoneContainer.GetSkeletalMeshAsset()->GetSkeleton();
		auto GetBasePoseTransformWithHistory = [this, PoseHistory, Skeleton, &BaseData, &BoneContainer, &Output](FCompactPoseBoneIndex CompactBoneIdx, FTransform & OutTransform)
		{
			if (PoseHistory)
			{
				const FSkeletonPoseBoneIndex SkeletonBoneIdx = BoneContainer.GetSkeletonPoseIndexFromCompactPoseIndex(CompactBoneIdx);
				if (PoseHistory->GetTransformAtTime(-Delay, OutTransform, Skeleton, SkeletonBoneIdx.GetInt()))
				{
					return;
				}
				LogRequestError(Output, SkeletonBoneIdx);
			}
			// If no pose history is found, use current frame instead
			OutTransform = BaseData.Pose.GetComponentSpaceTransform(CompactBoneIdx);
		};

		GetBasePoseTransformWithHistory(CompactPoseSourceBoneIdx, SourceToTransform);
		FTransform SourceFromTransform = ReferencePoseData.Pose.GetComponentSpaceTransform(CompactPoseSourceBoneIdx);

		if (CopySpace.IsValidToEvaluate(BoneContainer))
		{
			const FCompactPoseBoneIndex CompactPoseReferenceBoneIdx = CopySpace.GetCompactPoseIndex(BoneContainer);

			FTransform BoneSpaceToTransform;
			GetBasePoseTransformWithHistory(CompactPoseReferenceBoneIdx, BoneSpaceToTransform);

			FTransform BoneSpaceFromPoseTransform = ReferencePoseData.Pose.GetComponentSpaceTransform(CompactPoseReferenceBoneIdx);
		
			// If using root space, remove root rotations from four copy space transforms
			if (bUseRootSpace)
			{
				const FCompactPoseBoneIndex RootBoneIdx = FCompactPoseBoneIndex(0);

				FTransform RootToTransform;
				GetBasePoseTransformWithHistory(RootBoneIdx, RootToTransform);
				FQuat RootToRot = RootToTransform.GetRotation();
				if (!RootToRot.IsIdentity(SMALL_NUMBER))
				{
					const FQuat RootToRotInv = RootToRot.Inverse();
					UE::Anim::CopyBoneMotion::RemoveRootRotation(SourceToTransform, RootToRotInv);
					UE::Anim::CopyBoneMotion::RemoveRootRotation(BoneSpaceToTransform, RootToRotInv);
				}

				FQuat RootFromRot = ReferencePoseData.Pose.GetComponentSpaceTransform(RootBoneIdx).GetRotation();
				if (!RootFromRot.IsIdentity(SMALL_NUMBER))
				{
					const FQuat RootFromRotInv = RootFromRot.Inverse();
					UE::Anim::CopyBoneMotion::RemoveRootRotation(SourceFromTransform, RootFromRotInv);
					UE::Anim::CopyBoneMotion::RemoveRootRotation(BoneSpaceFromPoseTransform, RootFromRotInv);
				}
			}
			
			// Resulting translation move is an additive transform.
			 UpdateCopySpace(SourceToTransform, SourceFromTransform, BoneSpaceToTransform, BoneSpaceFromPoseTransform, TransformOffset);
		
		}
		else
		{
			// If using root space, remove root rotations from our transforms
			if (bUseRootSpace)
			{
				const FCompactPoseBoneIndex RootBoneIdx = FCompactPoseBoneIndex(0);
				FTransform RootToTransform;
				GetBasePoseTransformWithHistory(RootBoneIdx, RootToTransform);
				FQuat RootToRot = RootToTransform.GetRotation();

				if (!RootToRot.IsIdentity(SMALL_NUMBER))
				{
					UE::Anim::CopyBoneMotion::RemoveRootRotation(SourceToTransform, RootToRot.Inverse());
				}

				FQuat RootFromRot = ReferencePoseData.Pose.GetComponentSpaceTransform(RootBoneIdx).GetRotation();
				if (!RootFromRot.IsIdentity(SMALL_NUMBER))
				{
					UE::Anim::CopyBoneMotion::RemoveRootRotation(SourceFromTransform, RootFromRot.Inverse());
				}
			}

			// No reference bone. Compute transform in component space
			TransformOffset = UE::Anim::CopyBoneMotion::GetTransformDelta(SourceFromTransform, SourceToTransform);
		}

		if (UE::Anim::CopyBoneMotion::ShouldShowCopiedMotion())
		{
			// Reset pose to input reference pose, to only show copied motion, and no inherited motion.
			Output = ReferencePoseData;
		}
	}
	else
	{
		//@todo: Curves may not be the best way to store this information vs a bone or a transform attribute.
		// Grab curve values and scale by the node's modifiers
		TransformOffset.Location.X = Output.Curve.Get(TranslationX_CurveName);
		TransformOffset.Location.Y = Output.Curve.Get(TranslationY_CurveName);
		TransformOffset.Location.Z = Output.Curve.Get(TranslationZ_CurveName);

		RotationOffset.Pitch = Output.Curve.Get(RotationPitch_CurveName);
		RotationOffset.Yaw = Output.Curve.Get(RotationYaw_CurveName);
		RotationOffset.Roll = Output.Curve.Get(RotationRoll_CurveName);
	}

	if (HasTargetCurve())
	{
		// Output the motion to the specified curve as a scalar value.
		const float CurrentTargetCurveValue = Output.Curve.Get(TargetCurveName);
		const float TargetCurveMotionDelta = GetTargetCurveValue(TransformOffset, TargetCurveComponent);
		Output.Curve.Set(TargetCurveName, CurrentTargetCurveValue + TargetCurveMotionDelta * TargetCurveScale);
	}

	if (!HasTargetBone(BoneContainer))
	{
		// No target bone defined. No need to apply this transform.
		return;
	}

	// Scale translation
	if (TranslationRemapCurve)
	{
		TArray<FRichCurveEditInfoConst, TInlineAllocator<16>> Curves;
		TranslationRemapCurve->GetCurves(Curves);
		TransformOffset.Location.X = Curves[0].CurveToEdit->Eval(TransformOffset.Location.X);
		TransformOffset.Location.Y = Curves[1].CurveToEdit->Eval(TransformOffset.Location.Y);
		TransformOffset.Location.Z = Curves[2].CurveToEdit->Eval(TransformOffset.Location.Z);
	}
	TransformOffset.Location *= TranslationScale;

	// Scale rotation
	{
		float Angle;
		FVector Axis;
		TransformOffset.Rotation.ToAxisAndAngle(Axis, Angle);
		if (RotationRemapCurve)
		{
			Angle = RotationRemapCurve->GetFloatValue(Angle);
		}
		TransformOffset.Rotation = FQuat(Axis, Angle * RotationScale);
	}

	const FCompactPoseBoneIndex CompactPoseBoneToModify = BoneToModify.GetCompactPoseIndex(BoneContainer);
	const FTransform InitialBoneToModifyTM = Output.Pose.GetComponentSpaceTransform(CompactPoseBoneToModify);
	FTransform NewBoneTM = InitialBoneToModifyTM;

#if ENABLE_ANIM_DEBUG
	const FTransform& ComponentTransform = Output.AnimInstanceProxy->GetComponentTransform();
	const FTransform OriginalBoneWS = NewBoneTM * ComponentTransform;
	ApplySpaceDebugTransform = FTransform::Identity;
#endif

	TransformOffset.Location = TranslationOffset.RotateVector(TransformOffset.Location);
	TransformOffset.Rotation = UE::Anim::CopyBoneMotion::ReOrientDeltaQuat(TransformOffset.Rotation, FQuat(RotationOffset));
	
	UpdateApplySpace(Output, BoneContainer, CompactPoseBoneToModify, TransformOffset, NewBoneTM);
	
	const float DeltaSeconds = Output.AnimInstanceProxy->GetDeltaSeconds();
	if (bDeltaSmoothing)
	{
		FTransform AdditiveTransform = NewBoneTM;
		// Generate an additive from AdditiveT = (NewT - InitialT)
		FAnimationRuntime::ConvertTransformToAdditive(AdditiveTransform, InitialBoneToModifyTM);

		if (bIsFirstUpdate || (TranslationSmoothingTime <= 0.0f && RotationSmoothingTime <= 0.0f))
		{
			SpringState.CurrentTransform = AdditiveTransform;
			SpringState.Velocity = FVector::ZeroVector;
			SpringState.AngularVelocity = FVector::ZeroVector;
		}
		else
		{
			// This will run unneccesarily if only one smoothing time == 0.0f, but damper snaps in that case anyway.
			SpringMath::CriticalSpringDamper(SpringState.CurrentTransform.Location, SpringState.Velocity, AdditiveTransform.GetTranslation(), TranslationSmoothingTime, DeltaSeconds);
			SpringMath::CriticalSpringDamperQuat(SpringState.CurrentTransform.Rotation, SpringState.AngularVelocity, AdditiveTransform.GetRotation(), RotationSmoothingTime, DeltaSeconds);
			
			// Apply the additive as NewT = (InitialT + AdditiveT)
			NewBoneTM = InitialBoneToModifyTM;

			const ScalarRegister AdditiveWeight(1.f);
			NewBoneTM.Accumulate(SpringState.CurrentTransform, AdditiveWeight);
		}
	}
	else
	{
		if (bIsFirstUpdate || (TranslationSmoothingTime <= 0.0f && RotationSmoothingTime <= 0.0f))
		{
			SpringState.CurrentTransform = NewBoneTM;
			SpringState.Velocity = FVector::ZeroVector;
			SpringState.AngularVelocity = FVector::ZeroVector;
		}
		else
		{
			// This will run unneccesarily if only one smoothing time == 0.0f, but damper snaps in that case anyway.
			SpringMath::CriticalSpringDamper(SpringState.CurrentTransform.Location, SpringState.Velocity, NewBoneTM.GetTranslation(), TranslationSmoothingTime, DeltaSeconds);
			SpringMath::CriticalSpringDamperQuat(SpringState.CurrentTransform.Rotation, SpringState.AngularVelocity, NewBoneTM.GetRotation(), RotationSmoothingTime, DeltaSeconds);
			NewBoneTM.SetTranslation(SpringState.CurrentTransform.Location);
			NewBoneTM.SetRotation(SpringState.CurrentTransform.Rotation);
		}
	}

	if (UE::Anim::CopyBoneMotion::IsDebugEnabled())
	{
#if ENABLE_ANIM_DEBUG
		const FQuat& DebugApplySpaceOrientation = (ApplySpace.IsValidToEvaluate(BoneContainer)) ? OriginalBoneWS.GetRotation() : ApplySpaceDebugTransform.GetRotation() * ComponentTransform.GetRotation();
		const FTransform TargetBoneTransformWS = NewBoneTM * ComponentTransform;
		// Draw coordinate system at original bone location
		Output.AnimInstanceProxy->AnimDrawDebugCoordinateSystem(OriginalBoneWS.GetLocation(), DebugApplySpaceOrientation.Rotator(), 15.0f, false, -1.0f, 2.0f, SDPG_Foreground);
		// Draw line between original and new bone locations. Original bone serves as pivot
		Output.AnimInstanceProxy->AnimDrawDebugLine(OriginalBoneWS.GetLocation(), TargetBoneTransformWS.GetLocation(), FColor::Yellow, false, -1.0f, 1.0f, SDPG_Foreground);

		float Angle;
		FVector Axis;
		TransformOffset.Rotation.ToAxisAndAngle(Axis, Angle);
		// Draw hinge axis and hinge coordinate system
		Output.AnimInstanceProxy->AnimDrawDebugLine(TargetBoneTransformWS.GetLocation(), TargetBoneTransformWS.GetLocation() + DebugApplySpaceOrientation.RotateVector(Axis) * 10.0f, FColor::Yellow, false, -1.0f, 1.0f, SDPG_Foreground);
		Output.AnimInstanceProxy->AnimDrawDebugCoordinateSystem(TargetBoneTransformWS.GetLocation(), DebugApplySpaceOrientation.Rotator(), 10.0f, false, -1.0f, 1.0f, SDPG_Foreground);

		const bool bShouldDrawMotionHistory = UE::Anim::CopyBoneMotion::ShouldDrawMotionHistory();
		const FVector LocDebug = TargetBoneTransformWS.GetLocation();
		const FVector RotDebug = LocDebug + DebugApplySpaceOrientation.RotateVector(Axis) * 2.0f;

		if (bShouldDrawMotionHistory)
		{
			// Only draw if we haven't just enabled debug draw.
			if ((bIsDrawHistoryEnabled == bShouldDrawMotionHistory)
				&& !bIsFirstUpdate)
			{
				// Draw a quad in the direction of our hinge axis
				Output.AnimInstanceProxy->AnimDrawDebugLine(LocDebug, RotDebug, FColor::Cyan, false, 2.0f, 0.05f, SDPG_World);
				Output.AnimInstanceProxy->AnimDrawDebugLine(LastRotDebug, LastLocDebug, FColor::Cyan, false, 2.0f, 0.05f, SDPG_World);
				Output.AnimInstanceProxy->AnimDrawDebugLine(LastLocDebug, RotDebug, FColor::Cyan, false, 2.0f, 0.05f, SDPG_World);
				Output.AnimInstanceProxy->AnimDrawDebugLine(LastRotDebug, RotDebug, FColor::Cyan, false, 2.0f, 0.05f, SDPG_World);

				// Draw a line between move deltas.
				Output.AnimInstanceProxy->AnimDrawDebugLine(LocDebug, LastLocDebug, FColor::Yellow, false, 2.0f, 0.2f, SDPG_World);
			}
		}
		LastRotDebug = RotDebug;
		LastLocDebug = LocDebug;

		bIsDrawHistoryEnabled = bShouldDrawMotionHistory;
#endif
	}

	OutBoneTransforms.Add( FBoneTransform(BoneToModify.GetCompactPoseIndex(BoneContainer), NewBoneTM) );
	bIsFirstUpdate = false;
}

void FAnimNode_CopyBoneMotion::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	Super::CacheBones_AnyThread(Context);

	if (bUseBasePose)
	{
		BasePose.CacheBones(Context);
		BasePoseReference.CacheBones(Context);
	}
}

void FAnimNode_CopyBoneMotion::UpdateComponentPose_AnyThread(const FAnimationUpdateContext& Context)
{
	Super::UpdateComponentPose_AnyThread(Context);

	if (bUseBasePose)
	{
		BasePose.Update(Context);
		BasePoseReference.Update(Context);
	}
}

bool FAnimNode_CopyBoneMotion::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	if ( UE::Anim::CopyBoneMotion::CVarEnable.GetValueOnAnyThread() == 0)
	{
		return false;
	}

	if (!HasTargetBone(RequiredBones) && !HasTargetCurve())
	{
		return false;
	}

	if ((ApplySpace.BoneIndex != INDEX_NONE) && (ApplySpace.IsValidToEvaluate(RequiredBones) == false))
	{
		return false;
	}

	if (bUseBasePose)
	{
		if (SourceBone.IsValidToEvaluate(RequiredBones) == false)
		{
			return false;
		}

		if ((CopySpace.BoneIndex != INDEX_NONE) && (CopySpace.IsValidToEvaluate(RequiredBones) == false))
		{
			return false;
		}
	}

	return true;
}

void FAnimNode_CopyBoneMotion::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	Super::InitializeBoneReferences(RequiredBones);

	BoneToModify.Initialize(RequiredBones);
	SourceBone.Initialize(RequiredBones);
	CopySpace.Initialize(RequiredBones);
	ApplySpace.Initialize(RequiredBones);
}

float FAnimNode_CopyBoneMotion::GetTargetCurveValue(const FTransformNoScale& InTransformDelta, ECopyBoneMotion_Component MotionComponent)
{
	if (MotionComponent == ECopyBoneMotion_Component::RotationAngle)
	{
		const FVector TwistAxis = UE::Anim::CopyBoneMotion::GetAxisVector(TargetCurveRotationAxis);
		const float SignedAngle = UE::Anim::CopyBoneMotion::GetSignedAngleAroundAxis(InTransformDelta.Rotation, TwistAxis);
		return FMath::RadiansToDegrees(SignedAngle);
	}
	else
	{
		return InTransformDelta.Location[static_cast<int32>(MotionComponent)];
	}
}

bool FAnimNode_CopyBoneMotion::HasTargetBone(const FBoneContainer& BoneContainer) const
{
	return BoneToModify.IsValidToEvaluate(BoneContainer);
}

bool FAnimNode_CopyBoneMotion::HasTargetCurve() const
{
	return TargetCurveName != NAME_None;
}

void FAnimNode_CopyBoneMotion::Reset()
{
	bIsFirstUpdate = true;
}

void FAnimNode_CopyBoneMotion::UpdateApplySpace(FComponentSpacePoseContext& Output, const FBoneContainer& BoneContainer, FCompactPoseBoneIndex CompactPoseBoneToModify, FTransformNoScale& InOutTransformOffset, FTransform& InOutNewBoneTM)
{
	if ((InOutTransformOffset.Rotation.Rotator().IsNearlyZero() == false)
		|| (InOutTransformOffset.Location.IsNearlyZero() == false))
	{
		const bool bUseApplySpace = ApplySpace.IsValidToEvaluate(BoneContainer);
	
		if (bUseApplySpace)
		{
			const FCompactPoseBoneIndex CompactPoseApplySpaceIdx = ApplySpace.GetCompactPoseIndex(BoneContainer);
			
			const FTransform ApplySpaceBoneTransform = Output.Pose.GetComponentSpaceTransform(CompactPoseApplySpaceIdx);
#if ENABLE_ANIM_DEBUG
			ApplySpaceDebugTransform = ApplySpaceBoneTransform;
#endif
			// Find apply space delta in component space by comparing with ref pose
			const FTransform ApplySpaceRefPose = FAnimationRuntime::GetComponentSpaceRefPose(CompactPoseApplySpaceIdx, BoneContainer);
			FTransform DeltaApplySpace = UE::Anim::CopyBoneMotion::GetTransformDelta(ApplySpaceRefPose, ApplySpaceBoneTransform); 
			// Apply space is rotation only
			DeltaApplySpace.SetTranslation(FVector::ZeroVector);
			
			// Remove apply space from target bone transform to apply both transform offset and rotation pivot in apply space (we could rotate both transform offset and rotation pivot instead, but this is faster)
			InOutNewBoneTM = InOutNewBoneTM * DeltaApplySpace.Inverse();
			// Apply transformation with apply space removed
			InOutNewBoneTM.SetRotation(InOutTransformOffset.Rotation * InOutNewBoneTM.GetRotation());
			InOutNewBoneTM.AddToTranslation(InOutTransformOffset.Location);
			if (RotationPivot.IsNearlyZero() == false)
			{
				// Rotate our pivot offset by the rotation offset.
				// Apply the pivot delta to the target bone
				const FVector PivotDelta = InOutTransformOffset.Rotation.RotateVector(RotationPivot) - RotationPivot;
				InOutNewBoneTM.AddToTranslation(PivotDelta);
			}

			// Re-apply apply space
			InOutNewBoneTM = InOutNewBoneTM * DeltaApplySpace;

			//////////////////////////////////////////////////////////////////////////
		}
		else
		{
			// Apply motion in component space
			InOutNewBoneTM.SetRotation(InOutTransformOffset.Rotation * InOutNewBoneTM.GetRotation());
			InOutNewBoneTM.AddToTranslation(InOutTransformOffset.Location);
			if (RotationPivot.IsNearlyZero() == false)
			{
				// Rotate our pivot offset by the rotation offset.
				// Apply the pivot delta to the target bone
				const FVector PivotDelta = InOutTransformOffset.Rotation.RotateVector(RotationPivot) - RotationPivot;
				InOutNewBoneTM.AddToTranslation(PivotDelta);
			}
		}
	}
}

void FAnimNode_CopyBoneMotion::UpdateCopySpace(const FTransform& SourceToTransform, const FTransform& SourceFromTransform, const FTransform& BoneSpaceToTransform, const FTransform& BoneSpaceFromPoseTransform, FTransformNoScale& OutTransformOffset)
{
	// Compute motion delta between copy space and source bone in component space.
	FTransform DeltaCopySpace = UE::Anim::CopyBoneMotion::GetTransformDelta(BoneSpaceFromPoseTransform, BoneSpaceToTransform);
	FTransform DeltaSourceBone = UE::Anim::CopyBoneMotion::GetTransformDelta(SourceFromTransform, SourceToTransform);
	OutTransformOffset = UE::Anim::CopyBoneMotion::GetTransformDelta(DeltaCopySpace, DeltaSourceBone);
}

void FAnimNode_CopyBoneMotion::LogRequestError(const FComponentSpacePoseContext& Context, const FSkeletonPoseBoneIndex BoneIdx)
{
#if WITH_EDITORONLY_DATA	
	const int32 NodePropertyIndex = Context.GetCurrentNodeId();

	const FBoneContainer& BoneContainer = Context.Pose.GetPose().GetBoneContainer();
	const USkeleton* Skeleton = BoneContainer.GetSkeletalMeshAsset()->GetSkeleton();

	FName BoneName = Skeleton->GetReferenceSkeleton().GetBoneName(BoneIdx.GetInt());

	UAnimBlueprint* AnimBlueprint = Context.AnimInstanceProxy->GetAnimBlueprint();
	UAnimBlueprintGeneratedClass* AnimClass = AnimBlueprint ? AnimBlueprint->GetAnimBlueprintGeneratedClass() : nullptr;
	const UObject* RequesterNode = AnimClass ? AnimClass->GetVisualNodeFromNodePropertyIndex(NodePropertyIndex) : nullptr;

	Context.LogMessage(FTokenizedMessage::Create(EMessageSeverity::Error)
		->AddToken(FTextToken::Create(FText::Format(LOCTEXT("CopyBoneMotionPoseHistoryError",
			"PoseHistory lookup failed for bone {0} (delay: {1}). Verify bone is tracked and delay does not exceed time horizon. Source Node: "), FText::FromName(BoneName), Delay)))
		->AddToken(FUObjectToken::Create(RequesterNode)));
#endif
}

#undef LOCTEXT_NAMESPACE