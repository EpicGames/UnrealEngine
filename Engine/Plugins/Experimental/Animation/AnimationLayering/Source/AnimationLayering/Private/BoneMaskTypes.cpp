// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneMaskTypes.h"
#include "Animation/Skeleton.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoneMaskTypes)

#if WITH_EDITORONLY_DATA

namespace UE::Anim::BoneMask
{
	TAutoConsoleVariable<int32> CVarBoneMaskDataAssetForceRebuild(TEXT("a.AnimNode.Bonemask.DataAsset.ForceRebuild"), 0, TEXT(""));
}

void UBoneMaskDefinitionDataAsset::ConditionallyUpdateBoneMaskCachedData() const
{
	if (!Skeleton || !NeedsBoneMaskUpdate(*Skeleton))
	{
		return;
	}

	UpdateBoneMaskCachedData_Internal();
}

void UBoneMaskDefinitionDataAsset::UpdateBoneMaskCachedData_Internal() const
{
	if (Skeleton)
	{
		auto GetBoneWeightsForBodyPartToFill = [this](int32 BodyPartIdx) -> TArray<FBoneMaskPerBoneData>& { return BoneMaskDefinition.BodyPartDefinitions[BodyPartIdx].SkeletonPoseBoneWeights; };

		auto GetChildBoneIndicesForBodyPartToFill = [this](int32 BodyPartIdx) -> TArray<int32>& { return BoneMaskDefinition.BodyPartDefinitions[BodyPartIdx].SkeletonPoseChildBoneIndices; };

		UpdateBoneMaskCachedData(*Skeleton, BoneMaskDefinition, GetBoneWeightsForBodyPartToFill, GetChildBoneIndicesForBodyPartToFill);

		SkeletonGuid = Skeleton->GetGuid();
		VirtualBoneGuid = Skeleton->GetVirtualBoneGuid();
	}
}

// Check whether per-bone blend weights are valid according to the skeleton (GUID check)
bool UBoneMaskDefinitionDataAsset::NeedsBoneMaskUpdate(const USkeleton& InSkeleton) const
{
	return (UE::Anim::BoneMask::CVarBoneMaskDataAssetForceRebuild.GetValueOnAnyThread() > 0) || (InSkeleton.GetGuid() != SkeletonGuid) || (InSkeleton.GetVirtualBoneGuid() != VirtualBoneGuid);
}
#endif

void UBoneMaskDefinitionDataAsset::UpdateBoneMaskCachedData(const USkeleton& Skeleton, const FBoneMaskDefinition& BoneMaskDefinition, FGetBoneWeightsForBodyPartsCallback GetBoneWeightsForBodyPartsCallback, FGetChildBoneIndicesForBodyPartCallback GetChildBoneIndicesForBodyPartCallback)
{
	// @TODO: Unlikely to happen, but the data asset can be modified by game thread while during an anim update.
	// Lock the resource if this happens.
	// This is NOT a problem in a non-editor, because the data asset is strictly read-only in that case.

	const FReferenceSkeleton& RefSkeleton = Skeleton.GetReferenceSkeleton();
	const int32 NumBones = RefSkeleton.GetNum();

	TArray<float> FullSkeletonBoneWeights;
	FullSkeletonBoneWeights.Reserve(NumBones);
	for (int32 BodyPartIdx = 0; BodyPartIdx < BoneMaskDefinition.BodyPartDefinitions.Num(); ++BodyPartIdx)
	{
		const FBoneMaskBodyPartDefinition& BodyPartDef = BoneMaskDefinition.BodyPartDefinitions[BodyPartIdx];

		// Fill up a dense full skeleton array w/ weights. We'll turn it into sparse data later.
		FullSkeletonBoneWeights.Reset();
		FullSkeletonBoneWeights.AddZeroed(NumBones);

		for (int32 BranchIndex = 0; BranchIndex < BodyPartDef.BranchFilters.Num(); ++BranchIndex)
		{
			const FBranchFilter& BranchFilter = BodyPartDef.BranchFilters[BranchIndex];
			const int32 MaskBoneIndex = RefSkeleton.FindBoneIndex(BranchFilter.BoneName);

			if (MaskBoneIndex != INDEX_NONE)
			{
				// how much weight increase Per depth
				const float IncreaseWeightPerDepth = (BranchFilter.BlendDepth != 0) ? (1.f / ((float)BranchFilter.BlendDepth)) : 1.f;

				// go through skeleton bone hierarchy.
				// Bones are ordered, parents before children. So we can start looking at MaskBoneIndex for children.
				for (int32 BoneIndex = MaskBoneIndex; BoneIndex < NumBones; ++BoneIndex)
				{
					// if Depth == -1, it's not a child
					const int32 Depth = RefSkeleton.GetDepthBetweenBones(BoneIndex, MaskBoneIndex);
					if (Depth != -1)
					{
						const float BoneWeight = FMath::Clamp<float>((IncreaseWeightPerDepth * (float)(Depth + 1)), 0.f, 1.f);
						FullSkeletonBoneWeights[BoneIndex] = BoneWeight;
					}
				}
			}
		}

		// Turn our full skeleton bone weights array into sparse per-part data.
		TArray<FBoneMaskPerBoneData>& SkeletonPoseBoneWeights = GetBoneWeightsForBodyPartsCallback(BodyPartIdx);
		SkeletonPoseBoneWeights.Reset();
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			if (FAnimWeight::IsRelevant(FullSkeletonBoneWeights[BoneIndex]))
			{
				int32 BoneIndexInArray = SkeletonPoseBoneWeights.IndexOfByPredicate([BoneIndex](const FBoneMaskPerBoneData& BoneData) { return BoneData.SkeletonPoseBoneIndex == BoneIndex; });
				if (BoneIndexInArray == INDEX_NONE)
				{
					FBoneMaskPerBoneData BoneData;
					BoneData.BlendWeight = FullSkeletonBoneWeights[BoneIndex];
					BoneData.SkeletonPoseBoneIndex = BoneIndex;
					SkeletonPoseBoneWeights.Emplace(BoneData);
				}
			}
		}
	}

	for (int32 BodyPartIdx = 0; BodyPartIdx < BoneMaskDefinition.BodyPartDefinitions.Num(); ++BodyPartIdx)
	{
		TArray<FBoneMaskPerBoneData>& SkeletonPoseBoneWeights = GetBoneWeightsForBodyPartsCallback(BodyPartIdx);
		Algo::Sort(SkeletonPoseBoneWeights, [](const FBoneMaskPerBoneData& A, const FBoneMaskPerBoneData& B)
			{
				// Sort our elements from root to leaf.
				return A.SkeletonPoseBoneIndex < B.SkeletonPoseBoneIndex;
			});
	}

	for (int32 BodyPartIdx = 0; BodyPartIdx < BoneMaskDefinition.BodyPartDefinitions.Num(); ++BodyPartIdx)
	{
		TArray<FBoneMaskPerBoneData>& SkeletonPoseBoneWeights = GetBoneWeightsForBodyPartsCallback(BodyPartIdx);
		TArray<int32>& SkeletonPoseChildBoneIndices = GetChildBoneIndicesForBodyPartCallback(BodyPartIdx);
		SkeletonPoseChildBoneIndices.Reset();

		for (FBoneMaskPerBoneData& BoneData : SkeletonPoseBoneWeights)
		{
			TArray<int32> ChildBoneIndicesAsInt;
			RefSkeleton.GetDirectChildBones(BoneData.SkeletonPoseBoneIndex, ChildBoneIndicesAsInt);
			for (const int32 ChildIdx : ChildBoneIndicesAsInt)
			{
				if (SkeletonPoseBoneWeights.ContainsByPredicate([ChildIdx](const FBoneMaskPerBoneData& BoneData) { return BoneData.SkeletonPoseBoneIndex == ChildIdx; }) == false)
				{
					// Our body part bone indices are already sorted, so child insertion should be sorted too.
					SkeletonPoseChildBoneIndices.Add(ChildIdx);
				}
			}
		}
	}
}

void UBoneMaskDefinitionDataAsset::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.IsSaving())
	{
		// Make sure our cached data is up to date during save.
		ConditionallyUpdateBoneMaskCachedData();
	}
#endif

	// UPROPERTY serialization here.
	Super::Serialize(Ar);
}

const FBoneMaskDefinition& UBoneMaskDefinitionDataAsset::GetBoneMaskDefinition() const
{
#if WITH_EDITORONLY_DATA
	ConditionallyUpdateBoneMaskCachedData();
#endif

	return BoneMaskDefinition;
}

#if WITH_EDITOR
void UBoneMaskDefinitionDataAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	// Re-calculate our skeleton pose bone mask every time we change a value.
	// This should NOT be a slow operation, and should not be done synchronously if it ever becomes slow.
	
	UpdateBoneMaskCachedData_Internal();
}
#endif

void UBoneMaskDefinitionDataAsset::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	ConditionallyUpdateBoneMaskCachedData();
#endif
}
