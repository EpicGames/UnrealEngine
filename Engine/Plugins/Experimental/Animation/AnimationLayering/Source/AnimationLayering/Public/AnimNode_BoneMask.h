// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimNodeBase.h"
#include "BoneMaskTypes.h"
#include "AnimNode_BoneMask.generated.h"

#define UE_API ANIMATIONLAYERING_API

namespace UE::Anim::BoneMask
{
	struct FPerBoneData
	{
		FCompactPoseBoneIndex BoneIndex;
		float Weight;

		FPerBoneData(const FCompactPoseBoneIndex InBoneIndex, const float InWeight)
			: BoneIndex(InBoneIndex)
			, Weight(InWeight)
		{

		}
	};

	struct FBodyPartData
	{
		TArray<FPerBoneData> PerBoneData;
		TArray<FCompactPoseBoneIndex> ChildIndices;
		int32 PartIndex;
		int32 SourcePoseIndex;
		float MeshSpaceWeight;
		float LocalSpaceWeight;

#if WITH_EDITORONLY_DATA
		FName DebugPartName;
#endif
	};
}

USTRUCT(BlueprintType)
struct FBoneMaskBodyPartNameContainer
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Defaults")
	TArray<FName> Names;
};

/**
 * This node is heavily based on AnimNode_LayeredBoneBlend but replacing both BranchFilter and BlendMask option with a single input param that represents different body parts.
 * Each body parts has a list of BranchFilter that maps to actual bones. Blending is done in the order the body parts are created and in local space first and mesh space after.
 * Unlike AnimNode_LayeredBoneBlend, with this node we can change the mask dynamically.
*/
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BoneMask : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()

public:

	/** The source pose */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links)
	FPoseLink BasePose;

	/** Each layer's blended pose */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, editfixedsize, Category = Links, meta = (BlueprintCompilerGeneratedDefaults))
	TArray<FPoseLink> BlendPoses;

	/** The weights of each layer */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, editfixedsize, Category = Runtime, meta = (BlueprintCompilerGeneratedDefaults, PinShownByDefault))
	TArray<float> BlendWeights;

	/** Mask with all the body part dynamic values */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Runtime, meta = (BlueprintCompilerGeneratedDefaults, PinShownByDefault))
	FBoneMask BoneMask;

	/** Mask with all the body part static values */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Runtime, meta = (PinShownByDefault))
	TObjectPtr<UBoneMaskDefinitionDataAsset> BoneMaskDefinitionDataAsset = nullptr;

	/**
	 * List of BodyParts that want to use different Poses
	 * When adding a new input pose to the node (right click on the node Add Input Pose)
	 * an entry to this array is automatically added allowing the user to specify the body part(s) that should use that pose
	 */
	UPROPERTY(EditAnywhere, editfixedsize, Category = Runtime)
	TArray<FBoneMaskBodyPartNameContainer> BodyParts;

	/** How to blend the layers together */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Config)
	TEnumAsByte<enum ECurveBlendOption::Type>	CurveBlendOption;

	// Process mesh space rotations in root space, effectively ignoring any root rotation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Config, meta = (PinHiddenByDefault))
	bool bProcessMeshSpaceAsRootSpace = false;

	bool bHasRelevantPoses;

	/*
	 * Max LOD that this node is allowed to run
	 * For example if you have LODThreadhold to be 2, it will run until LOD 2 (based on 0 index)
	 * when the component LOD becomes 3, it will stop update/evaluate
	 * currently transition would be issue and that has to be re-visited
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Performance, meta = (DisplayName = "LOD Threshold"))
	int32 LODThreshold;

	/**
	 * Whether this node should update bone weights and child bone indices at runtime or use the data cached in the BoneMaskDefinitionDataAsset.
	 * Useful for cases where the skeleton can change at runtime and is not possible to have a BoneMaskDefinitionDataAsset for each skeleton.
	*/
	UPROPERTY(EditAnywhere, Category = Performance)
	bool bUpdateSkeletonDataOnDemand = false;

protected:

	TArray<UE::Anim::BoneMask::FBodyPartData> BodyPartsData;

	// @TODO: Temp, for now so we can pass it to BlendAttributesPerBoneFilter.
	TArray<FPerBoneBlendWeight> CurrentBoneBlendWeightsForAttributes;

	// Per-curve source pose index
	TBaseBlendedCurve<FDefaultAllocator, UE::Anim::FCurveElementIndexed> CurvePoseSourceIndices;

	// Serial number of the required bones container
	uint16 RequiredBonesSerialNumber;

	TWeakObjectPtr<const class USkeleton> CachedSkeleton;

	struct FBodyPartSkeletonData
	{
		TArray<FBoneMaskPerBoneData> SkeletonPoseBoneWeights;
		TArray<int32> SkeletonPoseChildBoneIndices;
	};
	TArray<FBodyPartSkeletonData> BodyPartsSkeletonData;

public:
	FAnimNode_BoneMask()
		: CurveBlendOption(ECurveBlendOption::Override)
		, bHasRelevantPoses(false)
		, LODThreshold(INDEX_NONE)
		, RequiredBonesSerialNumber(0)
	{
	}

	// FAnimNode_Base interface
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	virtual int32 GetLODThreshold() const override { return LODThreshold; }
	// End of FAnimNode_Base interface

	void AddPose()
	{
		BlendWeights.Add(1.f);

		new (BlendPoses) FPoseLink();

		if (BlendPoses.Num() > 1)
		{
			BodyParts.AddDefaulted_GetRef().Names.Add(NAME_None);
		}
	}

	void RemovePose(int32 PoseIndex)
	{
		if (PoseIndex >= 1)
		{
			BlendWeights.RemoveAt(PoseIndex);
			BlendPoses.RemoveAt(PoseIndex);

			if (BodyParts.IsValidIndex(PoseIndex - 1))
			{
				BodyParts.RemoveAt(PoseIndex - 1);
			}
		}
	}

	// Invalidates the cached bone data so it is recalculated the next time this node is updated
	void InvalidateCachedBoneData() { RequiredBonesSerialNumber = 0; }

private:

	// Update cached data if required
	UE_API void UpdateCachedBoneData(const FBoneContainer& RequiredBones, const USkeleton* Skeleton);

	friend class UAnimGraphNode_BoneMask;
};

#undef UE_API
