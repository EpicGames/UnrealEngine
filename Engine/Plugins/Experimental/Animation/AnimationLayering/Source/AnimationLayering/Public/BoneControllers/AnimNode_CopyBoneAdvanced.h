// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_CopyBoneAdvanced.generated.h"

#define UE_API ANIMATIONLAYERING_API

/**
 *	Controller to copy a bone's transform to another one, with individual per-component alphas.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_CopyBoneAdvanced : public FAnimNode_SkeletalControlBase
{
	GENERATED_BODY()

	/** Source Bone Name to get transform from */
	UPROPERTY(EditAnywhere, Category = Copy)
	FBoneReference SourceBone;

	/** Name of bone to control. This is the main bone chain to modify from. */
	UPROPERTY(EditAnywhere, Category=Copy) 
	FBoneReference TargetBone;

	/** Per-axis translation weight to copy from source to target. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Copy, meta = (AllowPreserveRatio, PinHiddenByDefault))
	FVector TranslationWeight;

	/** Rotation angle weight to copy from source to target. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Copy, meta = (PinHiddenByDefault))
	float RotationWeight;

	/** Scale weight to copy from source to target (not separated per-axis to avoid non-uniform scales) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Copy, meta = (AllowPreserveRatio, PinHiddenByDefault))
	float ScaleWeight;

	/** Space to convert transforms into prior to copying components */
	UPROPERTY(EditAnywhere, Category = Copy)
	TEnumAsByte<EBoneControlSpace> ControlSpace;

	/** Name of the bone Translation Weight is relative to.
		If left empty, the axes are in source bone space.
		If unchecked, the axes are in component space. */
	UPROPERTY(EditAnywhere, Category=Copy, meta = (EditCondition = "bTranslationInCustomBoneSpace")) 
	FBoneReference TranslationSpaceBone;

	// If disabled, the translation weight axes are in component-space
	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bTranslationInCustomBoneSpace;

	// If disabled, the translation weight axes are in component-space
	UPROPERTY(EditAnywhere, Category = Advanced)
	bool bPropagateToChildren;

	UE_API FAnimNode_CopyBoneAdvanced();

	// FAnimNode_Base interface
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	UE_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	TArray<FCompactPoseBoneIndex> ChildBoneIndices;
};

#undef UE_API
