// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "UObject/ObjectPtr.h"
#include "TransformNoScale.h"
#include "AnimNode_CopyBoneMotion.generated.h"

#define UE_API ANIMATIONLAYERING_API

 class UCurveVector;
 class UCurveFloat;

UENUM(BlueprintType)
enum class ECopyBoneMotion_Component : uint8
{
	TranslationX = 0,
	TranslationY = 1,
	TranslationZ = 2,
	// In Degrees
	RotationAngle = 3,
};

// Highly experimental plugin. There's a strong chance that this gets replaced with Control Rig functionality after initial exploration.
USTRUCT(BlueprintInternalUseOnly, Experimental)
struct FAnimNode_CopyBoneMotion : public FAnimNode_SkeletalControlBase
{
	GENERATED_BODY();

public:

	UE_API FAnimNode_CopyBoneMotion();

	// FAnimNode_Base interface
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	UE_API virtual void UpdateInternal(const FAnimationUpdateContext& Context) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	UE_API virtual void UpdateComponentPose_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context);
	UE_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	// Input pose to copy the motion from.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FComponentSpacePoseLink BasePose;

	// Reference pose used to calculate a motion delta from the base pose.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Links)
	FComponentSpacePoseLink BasePoseReference;

	// Whether to use the input Base Pose. Curves will be used when this is disabled.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Copy)
	bool bUseBasePose = false;

	// Tag of the pose history node to use to reference past bone transforms.
	// Your Pose History must include the desired bones used as Source and Copy.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Copy)
	FName PoseHistoryTag = NAME_None;

	// How much to delay the motion we're copying.
	// Your Pose History time horizon/duration must be at least this long.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Copy, meta = (ClampMin="0", PinHiddenByDefault))
	float Delay = 0.0f;

	// Bone to copy the motion from.
	UPROPERTY(EditAnywhere, Category = Copy, meta = (EditCondition = "bUseBasePose"))
	FBoneReference SourceBone;

	/** Name of bone to control. This is the main bone chain to modify from. **/
	UPROPERTY(EditAnywhere, Category = Copy)
	FBoneReference BoneToModify;

	/** Bone to use as the reference frame/space for our copied transform delta.
	If no reference frame is used, the source bone motion will be copied in component space. */
	UPROPERTY(EditAnywhere, Category = Copy, meta = (EditCondition = "bUseBasePose"))
	FBoneReference CopySpace;

	// Process CopySpace in root space, ignoring root bone rotation when computing the motion delta (Apply space is not impacted by root rotations)
	UPROPERTY(EditAnywhere, Category = Copy, meta = (PinHiddenByDefault, EditCondition = "bUseBasePose"))
	bool bUseRootSpace = false;

	/** Bone to use as the reference frame/space for our applied transform delta.
		If no reference frame is used, the source bone motion will be applied in component space. */
	UPROPERTY(EditAnywhere, Category = Copy)
	FBoneReference ApplySpace;

	/** Offset to use before applying the translation deltas (in degrees).
		This is useful for changing the direction of motion, relative to our reference frame/bone */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Translation, meta=(PinHiddenByDefault))
	FRotator TranslationOffset = FRotator::ZeroRotator;

	/** Rotation offset (in degrees) to apply before the rotation deltas.
		This is useful for changing the direction of motion, relative to our reference frame/bone */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Rotation, meta=(PinHiddenByDefault))
	FRotator RotationOffset = FRotator::ZeroRotator;

	/** Pivot offset (in local space) to use when applying the rotation.
		Any non-zero value will cause the target bone to rotate around the pivot, effectively introducing additional translation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Rotation, meta=(PinHiddenByDefault))
	FVector RotationPivot = FVector::ZeroVector;

	/** Curve prefix used for the animation curves. Format matches those generated by the LayeringMotionExtractorModifier */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Copy, meta = (EditCondition = "!bUseBasePose"))
	FName CurvePrefix = NAME_None;

	/** Name of the curve we're outputting motion to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Curve)
	FName TargetCurveName = NAME_None;

	/** Which component of motion we're outputting to the curve. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Curve, meta=(PinHiddenByDefault))
	float TargetCurveScale = 1.0f;

	/** Which component of motion we're outputting to the curve. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Curve)
	ECopyBoneMotion_Component TargetCurveComponent = ECopyBoneMotion_Component::RotationAngle;

	/** Axis around which to consider the rotation angle for the curve output. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Curve, meta = (EditCondition = "TargetCurveComponent == ECopyBoneMotion_Component::RotationAngle"))
	TEnumAsByte<EAxis::Type> TargetCurveRotationAxis = EAxis::X;

	UPROPERTY()
	FName TranslationX_CurveName = NAME_None;
	
	UPROPERTY()
	FName TranslationY_CurveName = NAME_None;

	UPROPERTY()
	FName TranslationZ_CurveName = NAME_None;

	UPROPERTY()
	FName RotationRoll_CurveName = NAME_None;

	UPROPERTY()
	FName RotationPitch_CurveName = NAME_None;

	UPROPERTY()
	FName RotationYaw_CurveName = NAME_None;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Translation, meta = (AllowPreserveRatio, PinHiddenByDefault))
	FVector TranslationScale = FVector::One();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Translation, meta = (PinHiddenByDefault))
	float TranslationSmoothingTime = 0.0f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Translation, meta = (PinHiddenByDefault))
	TObjectPtr<const UCurveVector> TranslationRemapCurve = nullptr;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Rotation, meta = (PinHiddenByDefault))
	float RotationScale = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Rotation, meta = (PinHiddenByDefault))
	float RotationSmoothingTime = 0.0f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Rotation, meta = (PinHiddenByDefault))
	TObjectPtr<const UCurveFloat> RotationRemapCurve = nullptr;

	// Do the smoothing as a delta between input and output poses, instead of a global component space smoothing.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Advanced)
	bool bDeltaSmoothing = false;

protected:

	UE_API void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	UE_API float GetTargetCurveValue(const FTransformNoScale& InTransformDelta, ECopyBoneMotion_Component MotionComponent);
	UE_API bool HasTargetBone(const FBoneContainer& BoneContainer) const;
	UE_API bool HasTargetCurve() const;
	UE_API void Reset();
	bool bIsFirstUpdate = true;

private:

#if ENABLE_ANIM_DEBUG
	FVector LastLocDebug = FVector::Zero();
	FVector LastRotDebug = FVector::Zero();
	FTransform ApplySpaceDebugTransform = FTransform::Identity;
	bool bIsDrawHistoryEnabled = false;
#endif
	
	void UpdateApplySpace(FComponentSpacePoseContext& Output, const FBoneContainer& BoneContainer, FCompactPoseBoneIndex CompactPoseBoneToModify, FTransformNoScale& InOutTransformOffset, FTransform& InOutNewBoneTM);
	void UpdateCopySpace(const FTransform& SourceToTransform, const FTransform& SourceFromTransform, const FTransform& BoneSpaceToTransform, const FTransform& BoneSpaceFromPoseTransform, FTransformNoScale& OutTransformOffset);

	struct FSpringData
	{
		FTransformNoScale CurrentTransform;
		FVector Velocity;
		FVector AngularVelocity;
	} SpringState;

	FGraphTraversalCounter UpdateCounter;
	
	void LogRequestError(const FComponentSpacePoseContext& Context, const FSkeletonPoseBoneIndex BoneIdx);

};

#undef UE_API
