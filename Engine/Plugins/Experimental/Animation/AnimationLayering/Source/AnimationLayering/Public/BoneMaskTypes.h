// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimData/BoneMaskFilter.h"
#include "Animation/AnimTypes.h"
#include "Engine/DataAsset.h"
#include "BoneMaskTypes.generated.h"

class USkeleton;

USTRUCT(BlueprintType)
struct FBoneMaskPerBoneData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = BoneMask)
	int32 SkeletonPoseBoneIndex = INDEX_NONE;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = BoneMask)
	float BlendWeight = INDEX_NONE;
};

USTRUCT(BlueprintType)
struct FBoneMaskBodyPartDefinition
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = BoneMask)
	FName Name = NAME_None;

	UPROPERTY(EditAnywhere, Category = BoneMask)
	TArray<FBranchFilter> BranchFilters;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = BoneMask)
	TArray<FBoneMaskPerBoneData> SkeletonPoseBoneWeights;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = BoneMask)
	TArray<int32> SkeletonPoseChildBoneIndices;
};

USTRUCT(BlueprintType)
struct FBoneMaskEntry
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BoneMask, meta = (UIMin = "0", UIMax = "1"))
	float LocalSpaceWeight = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BoneMask, meta = (UIMin = "0", UIMax = "1"))
	float MeshSpaceWeight = 0.0f;

	FBoneMaskEntry() = default;
	FBoneMaskEntry(const float InLocalSpaceWeight, const float InMeshSpaceWeight)
		: LocalSpaceWeight(InLocalSpaceWeight)
		, MeshSpaceWeight(InMeshSpaceWeight)
	{

	}
};

USTRUCT(BlueprintType)
struct FBoneMask
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BoneMask)
	TMap<FName, FBoneMaskEntry> BoneMaskMap;
};

USTRUCT(BlueprintType)
struct FBoneMaskDefinition
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = BoneMask)
	TArray<FBoneMaskBodyPartDefinition> BodyPartDefinitions;

	inline const FBoneMaskBodyPartDefinition* FindBodyPart(const FName& Name) const
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_BoneMask_FindBodyPart);

		// @TODO: This array is fairly small, but if this ended up being 'heavy' we may consider change to a map 
		return BodyPartDefinitions.FindByPredicate([&Name](const FBoneMaskBodyPartDefinition& Item) { return Item.Name == Name; });
	}

	inline FBoneMaskBodyPartDefinition* FindBodyPart(const FName& Name)
	{
		const FBoneMaskDefinition* ConstThis = this;
		return const_cast<FBoneMaskBodyPartDefinition*>(ConstThis->FindBodyPart(Name));
	}

	void AddBodyPartDefinition(const FName& Name, const TArray<FBranchFilter>& BranchFilters)
	{
		FBoneMaskBodyPartDefinition& NewDef = BodyPartDefinitions.AddDefaulted_GetRef();
		NewDef.Name = Name;
		NewDef.BranchFilters = BranchFilters;
	}
};

UCLASS(MinimalAPI, BlueprintType)
class UBoneMaskDefinitionDataAsset : public UDataAsset
{
	GENERATED_BODY()

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = BoneMask)
	TObjectPtr<USkeleton> Skeleton = nullptr;

	UPROPERTY()
	mutable FGuid SkeletonGuid;

	UPROPERTY()
	mutable FGuid VirtualBoneGuid;

	void ConditionallyUpdateBoneMaskCachedData() const;

	void UpdateBoneMaskCachedData_Internal() const;

	// Check whether per-bone blend weights are valid according to the skeleton (GUID check)
	bool NeedsBoneMaskUpdate(const USkeleton& InSkeleton) const;
#endif

protected:

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = BoneMask)
	mutable FBoneMaskDefinition BoneMaskDefinition;

public:

	ANIMATIONLAYERING_API void Serialize(FArchive& Ar);
	ANIMATIONLAYERING_API const FBoneMaskDefinition& GetBoneMaskDefinition() const;
	ANIMATIONLAYERING_API virtual void PostLoad() override;

	typedef TFunctionRef<TArray<FBoneMaskPerBoneData>& (int32 BodyPartIdx)> FGetBoneWeightsForBodyPartsCallback;
	typedef TFunctionRef<TArray<int32>& (int32 BodyPartIdx)> FGetChildBoneIndicesForBodyPartCallback;
	ANIMATIONLAYERING_API static void UpdateBoneMaskCachedData(const USkeleton& Skeleton, const FBoneMaskDefinition& BoneMaskDefinition, FGetBoneWeightsForBodyPartsCallback GetBoneWeightsForBodyPartsCallback, FGetChildBoneIndicesForBodyPartCallback GetChildBoneIndicesForBodyPartCallback);

#if WITH_EDITOR
	ANIMATIONLAYERING_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};

USTRUCT(BlueprintType)
struct FBoneMaskUpdateMultiParam
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FName Name = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	float LocalSpaceWeight = -1.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	float MeshSpaceWeight = -1.f;
};
