// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AnimationLayeringUncookedOnly : ModuleRules
{
    public AnimationLayeringUncookedOnly(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
	            "AnimationCore",
	            "AnimGraph",
	            "AnimGraphRuntime",
	            "AnimationLayering",
	            "Core",
	            "ToolMenus",
				"AnimationModifierLibrary",
			}
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "CoreUObject",
                "Engine",
                "Slate",
                "SlateCore",
                "PropertyEditor",
            }
        );
        
        if (Target.bBuildEditor == true)
        {
	        PrivateDependencyModuleNames.AddRange(
		        new string[]
		        {
			        "BlueprintGraph",
			        "EditorFramework",
			        "Kismet",
			        "UnrealEd",
		        }
	        );
        }
    }
}