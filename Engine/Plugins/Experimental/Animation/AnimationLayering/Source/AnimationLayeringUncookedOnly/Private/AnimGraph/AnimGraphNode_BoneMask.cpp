// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraph/AnimGraphNode_BoneMask.h"
#include "ToolMenus.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "AnimGraphCommands.h"
#include "ScopedTransaction.h"

#include "DetailLayoutBuilder.h"
#include "Kismet2/CompilerResultsLog.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_Mask

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BoneMask)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_BoneMask::UAnimGraphNode_BoneMask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Node.AddPose();
}

FLinearColor UAnimGraphNode_BoneMask::GetNodeTitleColor() const
{
	return FLinearColor(0.2f, 0.8f, 0.2f);
}

FText UAnimGraphNode_BoneMask::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_Mask_Tooltip", "Bone Mask");
}

FText UAnimGraphNode_BoneMask::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_Mask_Title", "Bone Mask");
}

void UAnimGraphNode_BoneMask::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FString UAnimGraphNode_BoneMask::GetNodeCategory() const
{
	return TEXT("Animation|Blends");
}

void UAnimGraphNode_BoneMask::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedRef<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(FName(TEXT("Node")), GetClass());

	Super::CustomizeDetails(DetailBuilder);
}

void UAnimGraphNode_BoneMask::PreloadRequiredAssets()
{
	Super::PreloadRequiredAssets();
}

void UAnimGraphNode_BoneMask::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	static auto AddInputPose = [this](const FToolMenuContext& InMenuContext)
	{
		UAnimGraphNode_BoneMask* MutableThis = const_cast<UAnimGraphNode_BoneMask*>(this);

		FScopedTransaction Transaction(LOCTEXT("AddPinToBoneMask", "AddPinToBoneMask"));
		MutableThis->Modify();

		MutableThis->Node.AddPose();
		MutableThis->ReconstructNode();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(MutableThis->GetBlueprint());
	};

	static auto RemoveInputPose = [this](const FToolMenuContext& InMenuContext, int32 Index)
	{
		UAnimGraphNode_BoneMask* MutableThis = const_cast<UAnimGraphNode_BoneMask*>(this);

		if (Index >= 1)
		{
			FScopedTransaction Transaction(LOCTEXT("RemovePinFromBoneMask", "RemovePinToBoneMask"));
			MutableThis->Modify();

			MutableThis->Node.RemovePose(Index);
			MutableThis->ReconstructNode();
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(MutableThis->GetBlueprint());
		}
	};

	if (!Context->bIsDebugging)
	{
		FToolMenuSection& Section = Menu->AddSection("AnimGraphNodeMask", LOCTEXT("Mask", "Mask"));
		if (Context->Pin != NULL)
		{
			if (Context->Pin->Direction == EGPD_Input)
			{
				FProperty* AssociatedProperty;
				int32 ArrayIndex;
				GetPinAssociatedProperty(GetFNodeType(), Context->Pin, AssociatedProperty, ArrayIndex);

				Section.AddMenuEntry(
					FName(TEXT("Remove Input Pose")),
					LOCTEXT("RemoveInputPose", "Remove Input Pose"),
					FText(),
					FSlateIcon(),
					FToolMenuExecuteAction::CreateLambda(RemoveInputPose, ArrayIndex));
			}
		}
		else
		{
			Section.AddMenuEntry(
				FName(TEXT("Add Input Pose")),
				LOCTEXT("AddInputPose", "Add Input Pose"),
				FText(),
				FSlateIcon(),
				FToolMenuExecuteAction::CreateLambda(AddInputPose));
		}
	}
}

void UAnimGraphNode_BoneMask::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	UAnimGraphNode_Base::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

void UAnimGraphNode_BoneMask::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
}

void UAnimGraphNode_BoneMask::PostLoad()
{
	Super::PostLoad();
}

#undef LOCTEXT_NAMESPACE
