// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraph/AnimGraphNode_CopyBoneAdvanced.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_CopyBoneAdvanced

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_CopyBoneAdvanced)

#define LOCTEXT_NAMESPACE "AnimationLayering"

FText UAnimGraphNode_CopyBoneAdvanced::GetControllerDescription() const
{
	return LOCTEXT("CopyBoneAdvanced", "Copy Bone Advanced");
}

FText UAnimGraphNode_CopyBoneAdvanced::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_CopyBoneAdvanced_Tooltip", "The Copy Bone control copies the Transform data or any component of it - i.e. Translation, Rotation, or Scale - from one bone to another.");
}

FText UAnimGraphNode_CopyBoneAdvanced::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if ((TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle) && (Node.TargetBone.BoneName == NAME_None) && (Node.SourceBone.BoneName == NAME_None))
	{
		return GetControllerDescription();
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDescription"), GetControllerDescription());
		Args.Add(TEXT("SourceBoneName"), FText::FromName(Node.SourceBone.BoneName));
		Args.Add(TEXT("TargetBoneName"), FText::FromName(Node.TargetBone.BoneName));

		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			return FText::Format(LOCTEXT("AnimGraphNode_CopyBoneAdvanced_ListTitle", "{ControllerDescription} - Source Bone: {SourceBoneName} - Target Bone: {TargetBoneName}"), Args);
		}
		else
		{
			return FText::Format(LOCTEXT("AnimGraphNode_CopyBoneAdvanced_Title", "{ControllerDescription}\nSource Bone: {SourceBoneName}\nTarget Bone: {TargetBoneName}"), Args);
		}
	}
}

void UAnimGraphNode_CopyBoneAdvanced::CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode)
{
	FAnimNode_CopyBoneAdvanced* CopyBoneNode = static_cast<FAnimNode_CopyBoneAdvanced*>(InPreviewNode);

	// copies Pin values from the internal node to get data that's compiled yet
	CopyBoneNode->TranslationWeight = Node.TranslationWeight;
	CopyBoneNode->RotationWeight = Node.RotationWeight;
	CopyBoneNode->ScaleWeight = Node.ScaleWeight;
}

void UAnimGraphNode_CopyBoneAdvanced::CopyPinDefaultsToNodeData(UEdGraphPin* InPin)
{
	if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CopyBoneAdvanced, TranslationWeight))
	{
		GetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CopyBoneAdvanced, TranslationWeight), Node.TranslationWeight);
	}
	else if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CopyBoneAdvanced, RotationWeight))
	{
		Node.RotationWeight = FCString::Atof(*InPin->GetDefaultAsString());
	}
	else if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CopyBoneAdvanced, ScaleWeight))
	{
		Node.ScaleWeight = FCString::Atof(*InPin->GetDefaultAsString());
	}
}

#undef LOCTEXT_NAMESPACE
