// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraph/AnimGraphNode_CopyBoneMotion.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"
#include "UnrealWidgetFwd.h"
#include "ScopedTransaction.h"
#include "MotionExtractorUtilities.h"
#include "MotionExtractorTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_CopyBoneMotion)

#define LOCTEXT_NAMESPACE "AnimationLayering"

FText UAnimGraphNode_CopyBoneMotion::GetControllerDescription() const
{
	return LOCTEXT("CopyBoneMotion", "Copy Bone Motion");
}

FText UAnimGraphNode_CopyBoneMotion::GetTooltipText() const
{
	return LOCTEXT("Copy Bone Motion", "Applies motion from a source, such as a curve, to a bone");
}

FText UAnimGraphNode_CopyBoneMotion::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetControllerDescription();
}

FLinearColor UAnimGraphNode_CopyBoneMotion::GetNodeTitleColor() const
{
	return FLinearColor(FColor(153.f, 0.f, 0.f));
}

void UAnimGraphNode_CopyBoneMotion::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);
}

void UAnimGraphNode_CopyBoneMotion::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
}

void UAnimGraphNode_CopyBoneMotion::CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode)
{
	FAnimNode_CopyBoneMotion* CopyMotionNode = static_cast<FAnimNode_CopyBoneMotion*>(InPreviewNode);

	// copies Pin values from the internal node to get data which are not compiled yet
	CopyMotionNode->TranslationScale = Node.TranslationScale;
	CopyMotionNode->TranslationRemapCurve = Node.TranslationRemapCurve;
	CopyMotionNode->TranslationOffset = Node.TranslationOffset;

	CopyMotionNode->RotationOffset = Node.RotationOffset;
	CopyMotionNode->RotationPivot = Node.RotationPivot;
	CopyMotionNode->RotationScale = Node.RotationScale;
	CopyMotionNode->RotationRemapCurve = Node.RotationRemapCurve;

	CopyMotionNode->Delay = Node.Delay;
}

void UAnimGraphNode_CopyBoneMotion::CopyPinDefaultsToNodeData(UEdGraphPin* InPin)
{
	if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CopyBoneMotion, TranslationScale))
	{
		GetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CopyBoneMotion, TranslationScale), Node.TranslationScale);
	}
	else if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CopyBoneMotion, TranslationOffset))
	{
		GetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CopyBoneMotion, TranslationOffset), Node.TranslationOffset);
	}
	else if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CopyBoneMotion, RotationScale))
	{
		Node.RotationScale = FCString::Atof(*InPin->GetDefaultAsString());
	}
	else if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CopyBoneMotion, RotationOffset))
	{
		GetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CopyBoneMotion, RotationOffset), Node.RotationOffset);
	}
	else if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CopyBoneMotion, RotationPivot))
	{
		GetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CopyBoneMotion, RotationPivot), Node.RotationPivot);
	}
}

void UAnimGraphNode_CopyBoneMotion::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	bool bShouldRebuildChain = false;

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FAnimNode_CopyBoneMotion, CurvePrefix))
	{
		FScopedTransaction Transaction(LOCTEXT("AnimGraphNode_ChangeCurvePrefix", "Change Curve Prefix"));
		Modify();

		// If the CurvePrefix changes, bake the new curve names on the node based on CurvePreflix w/ the format specified in LayeringMotionExtractorModifier
		Node.TranslationX_CurveName = UMotionExtractorUtilityLibrary::GenerateCurveName(Node.CurvePrefix, EMotionExtractor_MotionType::Translation, EMotionExtractor_Axis::X);
		Node.TranslationY_CurveName = UMotionExtractorUtilityLibrary::GenerateCurveName(Node.CurvePrefix, EMotionExtractor_MotionType::Translation, EMotionExtractor_Axis::Y);
		Node.TranslationZ_CurveName = UMotionExtractorUtilityLibrary::GenerateCurveName(Node.CurvePrefix, EMotionExtractor_MotionType::Translation, EMotionExtractor_Axis::Z);
		Node.RotationRoll_CurveName = UMotionExtractorUtilityLibrary::GenerateCurveName(Node.CurvePrefix, EMotionExtractor_MotionType::Rotation, EMotionExtractor_Axis::X);
		Node.RotationPitch_CurveName = UMotionExtractorUtilityLibrary::GenerateCurveName(Node.CurvePrefix, EMotionExtractor_MotionType::Rotation, EMotionExtractor_Axis::Y);
		Node.RotationYaw_CurveName = UMotionExtractorUtilityLibrary::GenerateCurveName(Node.CurvePrefix, EMotionExtractor_MotionType::Rotation, EMotionExtractor_Axis::Z);

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#undef LOCTEXT_NAMESPACE
