// Copyright Epic Games, Inc. All Rights Reserved.


#include "Modules/ModuleManager.h"
#include "BoneMaskDetailCustomization.h"
#include "PropertyEditorModule.h"

class FAnimationLayeringUncookedOnlyModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FAnimationLayeringUncookedOnlyModule, AnimationLayeringUncookedOnly)

void FAnimationLayeringUncookedOnlyModule::StartupModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomPropertyTypeLayout("BoneMaskEntry", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBoneMaskEntryDetails::MakeInstance));
}

void FAnimationLayeringUncookedOnlyModule::ShutdownModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.UnregisterCustomPropertyTypeLayout("BoneMaskEntry");    
}

