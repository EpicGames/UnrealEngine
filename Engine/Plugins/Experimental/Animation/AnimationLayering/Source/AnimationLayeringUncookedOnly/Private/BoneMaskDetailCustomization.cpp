// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneMaskDetailCustomization.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "BoneMaskTypes.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

//////////////////////////////////////////////////////////////////////////
// FBoneMaskEntryDetails
//////////////////////////////////////////////////////////////////////////
TSharedRef<IPropertyTypeCustomization> FBoneMaskEntryDetails::MakeInstance()
{
	return MakeShareable(new FBoneMaskEntryDetails());
}


void FBoneMaskEntryDetails::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> LocalWeightHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBoneMaskEntry, LocalSpaceWeight));
	TSharedPtr<IPropertyHandle> MeshWeightHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBoneMaskEntry, MeshSpaceWeight));

	HeaderRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];

	HeaderRow.ValueContent()
	[
		
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Local Space")))
			.Margin(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			.Font(CustomizationUtils.GetRegularFont())
		]
		+SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(140.0f)
				[
					LocalWeightHandle->CreatePropertyValueWidget()
				]
			]
		+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::FromString(TEXT("Mesh Space")))
			.Margin(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			.Font(CustomizationUtils.GetRegularFont())
			]
		+SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(140.0f)
				[
					MeshWeightHandle->CreatePropertyValueWidget()
				]
			]
	];
}

void FBoneMaskEntryDetails::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}