// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_CopyBoneAdvanced.h"
#include "AnimGraphNode_CopyBoneAdvanced.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_CopyBoneAdvanced : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_CopyBoneAdvanced Node;

public:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	// End of UEdGraphNode interface

protected:
	// UAnimGraphNode_SkeletalControlBase interface
	virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	virtual void CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode) override;
	virtual void CopyPinDefaultsToNodeData(UEdGraphPin* InPin) override;
	// End of UAnimGraphNode_SkeletalControlBase interface
};
