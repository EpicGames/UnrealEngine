// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_CopyBoneMotion.h"

#include "AnimGraphNode_CopyBoneMotion.generated.h"

#define UE_API ANIMATIONLAYERINGUNCOOKEDONLY_API

namespace ENodeTitleType { enum Type : int; }

UCLASS(MinimalAPI, Experimental)
class UAnimGraphNode_CopyBoneMotion : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_CopyBoneMotion Node;

public:

	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End of UEdGraphNode interface

protected:

	// UAnimGraphNode_Base interface
	UE_API virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	UE_API virtual void CopyNodeDataToPreviewNode(FAnimNode_Base* AnimNode) override;
	UE_API virtual void CopyPinDefaultsToNodeData(UEdGraphPin* InPin) override;
	// End of UAnimGraphNode_Base interface

	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	// UAnimGraphNode_SkeletalControlBase interface
	UE_API virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface
};

#undef UE_API
